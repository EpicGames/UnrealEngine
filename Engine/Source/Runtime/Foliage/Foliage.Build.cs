// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Foliage: ModuleRules
{
	public Foliage(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
                "Engine",
				"RenderCore",
				"RHI",
			}
		);
	}
}
