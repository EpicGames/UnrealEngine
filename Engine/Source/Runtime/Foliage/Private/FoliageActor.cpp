// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageActor.h"
#include "InstancedFoliageActor.h"
#include "FoliageType_Actor.h"
#include "FoliageHelper.h"
#include "Engine/Engine.h"

//
//
// FFoliageActor
void FFoliageActor::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	for (auto& Actor : ActorInstances)
	{
		if (Actor != nullptr)
		{
			Collector.AddReferencedObject(Actor, InThis);
		}
	}
}

void FFoliageActor::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (Ar.CustomVer(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::FoliageActorSupportNoWeakPtr)
	{
		Ar << ActorInstances_Deprecated;
	}
	else
#endif
	{
		Ar << ActorInstances;
	}

		
	Ar << ActorClass;
}

void FFoliageActor::DestroyActors(bool bOnLoad)
{
	TArray<AActor*> CopyActorInstances(ActorInstances);
	ActorInstances.Empty();
	for (AActor* Actor : CopyActorInstances)
	{
		if (Actor != nullptr)
		{
            if (bOnLoad)
			{
				Actor->ConditionalPostLoad();
			}
			Actor->GetWorld()->DestroyActor(Actor);
		}
	}
}

#if WITH_EDITOR
bool FFoliageActor::IsInitialized() const
{
	return ActorClass != nullptr;
}

void FFoliageActor::Initialize(const UFoliageType* FoliageType)
{
	check(!IsInitialized());
	const UFoliageType_Actor* FoliageType_Actor = Cast<UFoliageType_Actor>(FoliageType);
	ActorClass = FoliageType_Actor->ActorClass ? FoliageType_Actor->ActorClass.Get() : AActor::StaticClass();
	bShouldAttachToBaseComponent = FoliageType_Actor->bShouldAttachToBaseComponent;
}

void FFoliageActor::Uninitialize()
{
	check(IsInitialized());
	DestroyActors(false);
	ActorClass = nullptr;
}

AActor* FFoliageActor::Spawn(const FFoliageInstance& Instance)
{
	if (ActorClass == nullptr)
	{
		return nullptr;
	}

	AInstancedFoliageActor* IFA = GetIFA();

	FEditorScriptExecutionGuard ScriptGuard;
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.ObjectFlags = RF_Transactional;
	SpawnParameters.bHideFromSceneOutliner = true;
	SpawnParameters.bCreateActorPackage = false; // No OFPA because that would generate tons of files
	SpawnParameters.OverrideLevel = IFA->GetLevel();
	AActor* NewActor = IFA->GetWorld()->SpawnActor(ActorClass, nullptr, nullptr, SpawnParameters);
	if (NewActor)
	{
		NewActor->SetActorTransform(Instance.GetInstanceWorldTransform());
		FFoliageHelper::SetIsOwnedByFoliage(NewActor);
	}
	return NewActor;
}

TArray<AActor*> FFoliageActor::GetActorsFromSelectedIndices(const TSet<int32>& SelectedIndices) const
{
	TArray<AActor*> Selection;
	Selection.Reserve(SelectedIndices.Num());
	for (int32 i : SelectedIndices)
	{
		check(i < ActorInstances.Num());
		if (ActorInstances[i] != nullptr)
		{
			Selection.Add(ActorInstances[i]);
		}
	}

	return Selection;
}

int32 FFoliageActor::GetInstanceCount() const
{
	return ActorInstances.Num();
}

void FFoliageActor::PreAddInstances(const UFoliageType* FoliageType, int32 Count)
{
	if (!IsInitialized())
	{
		Initialize(FoliageType);
		check(IsInitialized());
	}
}

void FFoliageActor::AddInstance(const FFoliageInstance& NewInstance)
{
	ActorInstances.Add(Spawn(NewInstance));
}

void FFoliageActor::AddExistingInstance(const FFoliageInstance& ExistingInstance, UObject* InstanceImplementation)
{
	AActor* Actor = Cast<AActor>(InstanceImplementation);
	check(Actor);
	check(Actor->GetClass() == ActorClass);
	Actor->SetActorTransform(ExistingInstance.GetInstanceWorldTransform());
	FFoliageHelper::SetIsOwnedByFoliage(Actor);
	check(GetIFA()->GetLevel() == Actor->GetLevel());
	ActorInstances.Add(Actor);
}

void FFoliageActor::RemoveInstance(int32 InstanceIndex)
{
	if (ActorInstances.IsValidIndex(InstanceIndex))
	{
		AActor* Actor = ActorInstances[InstanceIndex];
		ActorInstances.RemoveAtSwap(InstanceIndex);

		if (Actor)
		{
			Actor->GetWorld()->DestroyActor(Actor, true);
			bActorsDestroyed = true;
		}
	}
}

void FFoliageActor::MoveInstance(int32 InstanceIndex, UObject*& OutInstanceImplementation)
{
	OutInstanceImplementation = nullptr;

	if (ActorInstances.IsValidIndex(InstanceIndex))
	{
		if (AActor* Actor = ActorInstances[InstanceIndex])
		{
			OutInstanceImplementation = Actor;
		}

		ActorInstances.RemoveAtSwap(InstanceIndex);
	}
}

void FFoliageActor::BeginUpdate()
{
	bActorsDestroyed = false;
}

void FFoliageActor::EndUpdate()
{
	if (bActorsDestroyed)
	{
		// This is to null out refs to components that have been created through ConstructionScript (same as it is done in edactDeleteSelected).
		// Because components that return true for IsCreatedByConstructionScript forward their Modify calls to their owning Actor so they are not part of the transaction.
		// Undoing the DestroyActor will re-run the construction script and those components will be recreated.
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}
	bActorsDestroyed = false;
}

void FFoliageActor::SetInstanceWorldTransform(int32 InstanceIndex, const FTransform& Transform, bool bTeleport)
{
	if (ActorInstances.IsValidIndex(InstanceIndex))
	{
		if (AActor* Actor = ActorInstances[InstanceIndex])
		{
			Actor->SetActorTransform(Transform);
		}
	}
}

FTransform FFoliageActor::GetInstanceWorldTransform(int32 InstanceIndex) const
{
	if (ActorInstances.IsValidIndex(InstanceIndex))
	{
		if (AActor* Actor = ActorInstances[InstanceIndex])
		{
			return Actor->GetTransform();
		}
	}

	return FTransform::Identity;
}

bool FFoliageActor::IsOwnedComponent(const UPrimitiveComponent* PrimitiveComponent) const
{
	const AActor* Owner = PrimitiveComponent->GetOwner();

	return ActorInstances.Contains(Owner);
}

int32 FFoliageActor::FindIndex(const AActor* InActor) const
{
	return ActorInstances.IndexOfByKey(InActor);
}

int32 FFoliageActor::GetInstanceIndexFrom(const UPrimitiveComponent* PrimitiveComponent, int32 ComponentIndex) const
{
	return FindIndex(PrimitiveComponent->GetOwner());
}

void FFoliageActor::Refresh(bool Async, bool Force)
{
	for (int32 i = 0; i < Info->Instances.Num(); ++i)
	{
		if (!IsValid(ActorInstances[i]))
		{
			ActorInstances[i] = Spawn(Info->Instances[i]);
		}
	}
}

void FFoliageActor::OnHiddenEditorViewMaskChanged(uint64 InHiddenEditorViews)
{
	for (AActor* Actor : ActorInstances)
	{
		if (Actor != nullptr)
		{
			if (Actor->HiddenEditorViews != InHiddenEditorViews)
			{
				Actor->HiddenEditorViews = InHiddenEditorViews;
				Actor->MarkComponentsRenderStateDirty();
			}
		}
	}
}

void FFoliageActor::UpdateActorTransforms(const TArray<FFoliageInstance>& Instances)
{
	for (int32 i = 0; i < Instances.Num(); ++i)
	{
		SetInstanceWorldTransform(i, Instances[i].GetInstanceWorldTransform(), true);
	}
}

void FFoliageActor::PostEditUndo(FFoliageInfo* InInfo, UFoliageType* FoliageType)
{
	// This will set Info to InInfo (as its probably nullptr after an undo operation)
	FFoliageImpl::PostEditUndo(InInfo, FoliageType);
	check(Info == InInfo);
	UpdateActorTransforms(Info->Instances);
}

void FFoliageActor::PreMoveInstances(TArrayView<const int32> InInstancesMoved)
{
	for (int32 Index : InInstancesMoved)
	{
		if (AActor* Actor = ActorInstances[Index])
		{
			Actor->Modify();
		}
	}
}

void FFoliageActor::PostMoveInstances(TArrayView<const int32> InInstancesMoved, bool bFinished)
{
	// Copy because moving actors might remove them from ActorInstances
	TArray<AActor*> MovedActors;
	MovedActors.Reserve(InInstancesMoved.Num());
	for (int32 Index : InInstancesMoved)
	{
		if (AActor * Actor = ActorInstances[Index])
		{
			MovedActors.Add(Actor);
			Actor->PostEditMove(bFinished);
		}
	}

	if (GIsEditor && GEngine && MovedActors.Num() && bFinished)
	{
		GEngine->BroadcastActorsMoved(MovedActors);
	}
}

bool FFoliageActor::NotifyFoliageTypeChanged(UFoliageType* FoliageType, bool bSourceChanged)
{
	if (!IsInitialized())
	{
		return false;
	}

	if (UFoliageType_Actor* FoliageTypeActor = Cast<UFoliageType_Actor>(FoliageType))
	{
		if (FoliageTypeActor->bStaticMeshOnly)
		{
			// requires implementation change
			return true;
		}

		AInstancedFoliageActor* IFA = GetIFA();
		if (bShouldAttachToBaseComponent != FoliageTypeActor->bShouldAttachToBaseComponent)
		{
			bShouldAttachToBaseComponent = FoliageTypeActor->bShouldAttachToBaseComponent;
			if (!bShouldAttachToBaseComponent)
			{
				IFA->RemoveBaseComponentOnFoliageTypeInstances(FoliageType);
			}
		}
	}

	if (bSourceChanged)
	{
		Reapply(FoliageType);
		ApplySelection(true, Info->SelectedIndices);
	}

	return false;
}

void FFoliageActor::Reapply(const UFoliageType* FoliageType)
{
	AInstancedFoliageActor* IFA = GetIFA();
	IFA->Modify();
	DestroyActors(false);
	
	if (IsInitialized())
	{
		Uninitialize();
	}
	Initialize(FoliageType);

	for (int32 i = 0; i < Info->Instances.Num(); ++i)
	{
		ActorInstances.Add(Spawn(Info->Instances[i]));
	}
}

void FFoliageActor::SelectAllInstances(bool bSelect)
{
	AInstancedFoliageActor::SelectionChanged.Broadcast(bSelect, ObjectPtrDecay(ActorInstances));
}

void FFoliageActor::SelectInstance(bool bSelect, int32 Index)
{
	if (ActorInstances.IsValidIndex(Index))
	{
		if (AActor* Actor = ActorInstances[Index])
		{
			TArray<AActor*> SingleInstance;
			SingleInstance.Add(Actor);
			AInstancedFoliageActor::SelectionChanged.Broadcast(bSelect, SingleInstance);
		}
	}
}

void FFoliageActor::SelectInstances(bool bSelect, const TSet<int32>& SelectedIndices)
{
	AInstancedFoliageActor::SelectionChanged.Broadcast(bSelect, GetActorsFromSelectedIndices(SelectedIndices));
}

FBox FFoliageActor::GetSelectionBoundingBox(const TSet<int32>& SelectedIndices) const
{
	FBox BoundingBox(EForceInit::ForceInit);
	TArray<AActor*> SelectedActors = GetActorsFromSelectedIndices(SelectedIndices);
	for (auto Actor : SelectedActors)
	{
		BoundingBox += Actor->GetComponentsBoundingBox();
	}
	return BoundingBox;
}
void FFoliageActor::ApplySelection(bool bApply, const TSet<int32>& SelectedIndices)
{
	if (bApply && SelectedIndices.Num() > 0)
	{
		AInstancedFoliageActor::SelectionChanged.Broadcast(true, GetActorsFromSelectedIndices(SelectedIndices));
	}
}

void FFoliageActor::ClearSelection(const TSet<int32>& SelectedIndices)
{
	AInstancedFoliageActor::SelectionChanged.Broadcast(false, GetActorsFromSelectedIndices(SelectedIndices));
}

bool FFoliageActor::UpdateInstanceFromActor(int32 Index, FFoliageInfo& FoliageInfo)
{
	if (ActorInstances.IsValidIndex(Index))
	{
		if (AActor* Actor = ActorInstances[Index])
		{
			AInstancedFoliageActor* IFA = GetIFA();
			IFA->Modify();
			const bool bChecked = false; // In the case of the PostEditUndo its possible that the instancehash is empty.
			FoliageInfo.InstanceHash->RemoveInstance(FoliageInfo.Instances[Index].Location, Index, bChecked);

			FTransform ActorTransform = Actor->GetTransform();
			FoliageInfo.Instances[Index].Location = ActorTransform.GetLocation();
			FoliageInfo.Instances[Index].Rotation = FRotator(ActorTransform.GetRotation());
			FoliageInfo.Instances[Index].PreAlignRotation = FoliageInfo.Instances[Index].Rotation;
			FoliageInfo.Instances[Index].DrawScale3D = (FVector3f)Actor->GetActorScale3D();
			FoliageInfo.InstanceHash->InsertInstance(FoliageInfo.Instances[Index].Location, Index);

			return true;
		}
	}

	return false;
}

void FFoliageActor::GetInvalidInstances(TArray<int32>& InvalidInstances)
{
	for (int32 Index = 0; Index < ActorInstances.Num(); ++Index)
	{
		if (ActorInstances[Index] == nullptr)
		{
			InvalidInstances.Add(Index);
		}
	}
}

#endif // WITH_EDITOR
