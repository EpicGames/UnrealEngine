// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "InstancedFoliage.h"

struct FFoliageActor : public FFoliageImpl
{
#if WITH_EDITORONLY_DATA
	TArray<TWeakObjectPtr<AActor>> ActorInstances_Deprecated;
#endif

	TArray<TObjectPtr<AActor>> ActorInstances;
	UClass* ActorClass;
	bool bShouldAttachToBaseComponent;
	
	FFoliageActor(FFoliageInfo* Info)
		: FFoliageImpl(Info)
		, ActorClass(nullptr)
		, bShouldAttachToBaseComponent(true)
#if WITH_EDITOR
		, bActorsDestroyed(false)
#endif
	{
	}

	virtual void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector) override;
	virtual void Serialize(FArchive& Ar) override;
	void DestroyActors(bool bOnLoad);

#if WITH_EDITOR
	virtual bool IsInitialized() const override;
	virtual void Initialize(const UFoliageType* FoliageType) override;
	virtual void Uninitialize() override;
	virtual int32 GetInstanceCount() const override;
	virtual void PreAddInstances(const UFoliageType* FoliageType, int32 Count) override;
	virtual void AddInstance(const FFoliageInstance& NewInstance) override;
	virtual void AddExistingInstance(const FFoliageInstance& ExistingInstance, UObject* InstanceImplementation) override;
	virtual void RemoveInstance(int32 InstanceIndex) override;
	virtual void MoveInstance(int32 InstanceIndex, UObject*& OutInstanceImplementation) override;
	virtual void SetInstanceWorldTransform(int32 InstanceIndex, const FTransform& Transform, bool bTeleport) override;
	virtual FTransform GetInstanceWorldTransform(int32 InstanceIndex) const override;
	virtual bool IsOwnedComponent(const UPrimitiveComponent* PrimitiveComponent) const override;
	int32 FindIndex(const AActor* InActor) const;
	virtual int32 GetInstanceIndexFrom(const UPrimitiveComponent* PrimitiveComponent, int32 ComponentIndex) const override;
	
	virtual void SelectAllInstances(bool bSelect) override;
	virtual void SelectInstance(bool bSelect, int32 Index) override;
	virtual void SelectInstances(bool bSelect, const TSet<int32>& SelectedIndices) override;
	virtual FBox GetSelectionBoundingBox(const TSet<int32>& SelectedIndices) const override;
	virtual void ApplySelection(bool bApply, const TSet<int32>& SelectedIndices) override;
	virtual void ClearSelection(const TSet<int32>& SelectedIndices) override;

	virtual void BeginUpdate() override;
	virtual void EndUpdate() override;
	virtual void Refresh(bool Async, bool Force) override;
	virtual void OnHiddenEditorViewMaskChanged(uint64 InHiddenEditorViews) override;
	virtual void PostEditUndo(FFoliageInfo* InInfo, UFoliageType* FoliageType) override;
	virtual void PreMoveInstances(TArrayView<const int32> InInstancesMoved) override;
	virtual void PostMoveInstances(TArrayView<const int32> InInstancesMoved, bool bFinished) override;
	virtual bool NotifyFoliageTypeChanged(UFoliageType* FoliageType, bool bSourceChanged) override;
	virtual void Reapply(const UFoliageType* FoliageType) override;
	AActor* Spawn(const FFoliageInstance& Instance);
	TArray<AActor*> GetActorsFromSelectedIndices(const TSet<int32>& SelectedIndices) const;
	virtual bool ShouldAttachToBaseComponent() const override { return bShouldAttachToBaseComponent; }
	bool UpdateInstanceFromActor(int32 Index, FFoliageInfo& FoliageInfo);
	void GetInvalidInstances(TArray<int32>& InvalidInstances);

private:
	void UpdateActorTransforms(const TArray<FFoliageInstance>& Instances);

	bool bActorsDestroyed;
#endif
};