// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FoliageComponent.cpp: Foliage rendering implementation.
=============================================================================*/

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "RenderingThread.h"
#include "GameFramework/Controller.h"
#include "Components/CapsuleComponent.h"
#include "GameFramework/Pawn.h"
#include "StaticMeshResources.h"
#include "StaticMeshSceneProxy.h"
#include "InteractiveFoliageActor.h"
#include "InteractiveFoliageComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/DamageEvents.h"

/** Scene proxy class for UInteractiveFoliageComponent. */
class FInteractiveFoliageSceneProxy final : public FStaticMeshSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FInteractiveFoliageSceneProxy(UInteractiveFoliageComponent* InComponent) :
		FStaticMeshSceneProxy(InComponent, false),
		FoliageImpluseDirection(0,0,0),
		FoliageNormalizedRotationAxisAndAngle(0,0,1,0)
	{}

	/** Accessor used by the rendering thread when setting foliage parameters for rendering. */
	void GetFoliageParameters(FVector& OutFoliageImpluseDirection, FVector4& OutFoliageNormalizedRotationAxisAndAngle) const
	{
		OutFoliageImpluseDirection = FoliageImpluseDirection;
		OutFoliageNormalizedRotationAxisAndAngle = FoliageNormalizedRotationAxisAndAngle;
	}

	/** Updates the scene proxy with new foliage parameters from the game thread. */
	void UpdateParameters_GameThread(const FVector& NewFoliageImpluseDirection, const FVector4& NewFoliageNormalizedRotationAxisAndAngle)
	{
		checkSlow(IsInGameThread());
		FInteractiveFoliageSceneProxy* FoliageProxy = this;
		ENQUEUE_RENDER_COMMAND(UpdateFoliageParameters)(
			[FoliageProxy, NewFoliageImpluseDirection, NewFoliageNormalizedRotationAxisAndAngle](FRHICommandListImmediate& RHICmdList)
			{
				FoliageProxy->FoliageImpluseDirection = NewFoliageImpluseDirection;
				FoliageProxy->FoliageNormalizedRotationAxisAndAngle = NewFoliageNormalizedRotationAxisAndAngle;
			});
	}

protected:

	FVector FoliageImpluseDirection;
	FVector4 FoliageNormalizedRotationAxisAndAngle;
};

UInteractiveFoliageComponent::UInteractiveFoliageComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FPrimitiveSceneProxy* UInteractiveFoliageComponent::CreateSceneProxy()
{
	// If a static mesh does not exist then this component cannot be added to the scene.
	if(GetStaticMesh() == NULL
		|| GetStaticMesh()->GetRenderData() == NULL
		|| GetStaticMesh()->GetRenderData()->LODResources.Num() == 0
		|| GetStaticMesh()->GetRenderData()->LODResources[0].VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() == 0)
	{
		return NULL;
	}

	// Store the foliage proxy so we can push updates to it during Tick
	FoliageSceneProxy = new FInteractiveFoliageSceneProxy(this);
	return FoliageSceneProxy;
}

void UInteractiveFoliageComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();
	FoliageSceneProxy = NULL;
}

float AInteractiveFoliageActor::TakeDamage(float DamageAmount, FDamageEvent const& DamageEvent, AController* EventInstigator, AActor* DamageCauser)
{
	FHitResult Hit;
	FVector ImpulseDir;
	DamageEvent.GetBestHitInfo(this, (EventInstigator ? EventInstigator->GetPawn() : NULL), Hit, ImpulseDir);

	// Discard the magnitude of the momentum and use Damage as the length instead
	FVector DamageImpulse = ImpulseDir.GetSafeNormal() * DamageAmount * FoliageDamageImpulseScale;

	// Apply force magnitude clamps
	DamageImpulse.X = FMath::Clamp<FVector::FReal>(DamageImpulse.X, -MaxDamageImpulse, MaxDamageImpulse);
	DamageImpulse.Y = FMath::Clamp<FVector::FReal>(DamageImpulse.Y, -MaxDamageImpulse, MaxDamageImpulse);
	DamageImpulse.Z = FMath::Clamp<FVector::FReal>(DamageImpulse.Z, -MaxDamageImpulse, MaxDamageImpulse);

	FoliageForce += DamageImpulse;
	
	// Bring this actor out of stasis so that it gets ticked now that a force has been applied
	SetActorTickEnabled(true);

	return 0.f;
}

void AInteractiveFoliageActor::CapsuleTouched(UPrimitiveComponent* OverlappedComp, AActor* Other, UPrimitiveComponent* OtherComp, int32 OtherBodyIndex, bool bFromSweep, const FHitResult& SweepResult)
{
	if (Other != NULL && OtherComp != NULL)
	{
		UCapsuleComponent* const TouchingActorCapsule = Cast<UCapsuleComponent>(OtherComp);
		if (TouchingActorCapsule && CapsuleComponent)
		{
			const FVector CenterToTouching = FVector(TouchingActorCapsule->Bounds.Origin.X, TouchingActorCapsule->Bounds.Origin.Y, CapsuleComponent->Bounds.Origin.Z) - CapsuleComponent->Bounds.Origin;
			// Keep track of the first position on the collision cylinder that the touching actor intersected
			//@todo - need to handle multiple touching actors
			TouchingActorEntryPosition = GetRootComponent()->Bounds.Origin + CenterToTouching.GetSafeNormal() * CapsuleComponent->GetScaledCapsuleRadius();
		}
		// Bring this actor out of stasis so that it gets ticked now that a force has been applied
		SetActorTickEnabled(true);
	}
}

void AInteractiveFoliageActor::SetupCollisionCylinder()
{
	if (GetStaticMeshComponent()->GetStaticMesh())
	{
		const FBoxSphereBounds MeshBounds = GetStaticMeshComponent()->GetStaticMesh()->GetBounds();
		const FVector Scale3D = GetStaticMeshComponent()->GetRelativeScale3D();
		// Set the cylinder's radius based off of the static mesh's bounds radius
		// CollisionRadius is in world space so apply the actor's scale
		CapsuleComponent->SetCapsuleSize(static_cast<float>(MeshBounds.SphereRadius * .7f * FMath::Max(Scale3D.X, Scale3D.Y)), static_cast<float>(MeshBounds.BoxExtent.Z * Scale3D.Z));


		// Ensure delegate is bound (just once)
		CapsuleComponent->OnComponentBeginOverlap.RemoveDynamic(this, &AInteractiveFoliageActor::CapsuleTouched);
		CapsuleComponent->OnComponentBeginOverlap.AddDynamic(this, &AInteractiveFoliageActor::CapsuleTouched);
	}
}

void AInteractiveFoliageActor::Tick(float DeltaSeconds)
{
	UInteractiveFoliageComponent* const FoliageComponent = CastChecked<UInteractiveFoliageComponent>(GetStaticMeshComponent());
	// Can only push updates to the scene proxy if we are being ticked while registered
	// The proxy will be NULL on dedicated server
	if (FoliageComponent->IsRegistered() && FoliageComponent->FoliageSceneProxy)
	{
		TSet<AActor*> Touching;
		GetOverlappingActors(Touching);

		for (AActor* TouchingActor : Touching)
		{
			if (TouchingActor != NULL && TouchingActor->GetRootComponent() != NULL)
			{
				const FVector TouchingActorPosition(TouchingActor->GetRootComponent()->Bounds.Origin.X, TouchingActor->GetRootComponent()->Bounds.Origin.Y, GetRootComponent()->Bounds.Origin.Z);
				//DrawDebugLine(GetWorld(), TouchingActorPosition, GetRootComponent()->Bounds.Origin, 255, 255, 255, false);
				// Operate on the touching actor's collision cylinder
				//@todo - handle touching actors without collision cylinders
				UCapsuleComponent* TouchingActorCapsule = Cast<UCapsuleComponent>(TouchingActor->GetRootComponent());
				if (TouchingActorCapsule && CapsuleComponent)
				{
					FVector TouchingToCenter = GetRootComponent()->Bounds.Origin - TouchingActorPosition;
					// Force the simulated position to be in the XY plane for simplicity
					TouchingToCenter.Z = 0;
					// Position on the collision cylinder mirrored across the cylinder's center from the position that the touching actor entered
					const FVector OppositeTouchingEntryPosition = GetRootComponent()->Bounds.Origin + GetRootComponent()->Bounds.Origin - TouchingActorEntryPosition;

					// Project the touching actor's center onto the vector from where it first entered to OppositeTouchingEntryPosition
					// This results in the same directional force being applied for the duration of the other actor touching this foliage actor,
					// Which prevents strange movement that results from just comparing cylinder centers.
					const FVector ProjectedTouchingActorPosition = (TouchingActorPosition - OppositeTouchingEntryPosition).ProjectOnTo(TouchingActorEntryPosition - OppositeTouchingEntryPosition) + OppositeTouchingEntryPosition;
					// Find the furthest position on the cylinder of the touching actor from OppositeTouchingEntryPosition
					const FVector TouchingActorFurthestPosition = ProjectedTouchingActorPosition + (TouchingActorEntryPosition - OppositeTouchingEntryPosition).GetSafeNormal() * TouchingActorCapsule->GetScaledCapsuleRadius();
					// Construct the impulse as the distance between the furthest cylinder positions minus the two cylinder's diameters
					const FVector ImpulseDirection = 
						- (OppositeTouchingEntryPosition - TouchingActorFurthestPosition 
						- (OppositeTouchingEntryPosition - TouchingActorFurthestPosition).GetSafeNormal() * 2.0f * (TouchingActorCapsule->GetScaledCapsuleRadius() + CapsuleComponent->GetScaledCapsuleRadius()));

					//DrawDebugLine(GetWorld(), GetRootComponent()->Bounds.Origin + FVector(0,0,100), GetRootComponent()->Bounds.Origin + ImpulseDirection + FVector(0,0,100), 100, 255, 100, false);

					// Scale and clamp the touch force
					FVector Impulse = ImpulseDirection * FoliageTouchImpulseScale;
					Impulse.X = FMath::Clamp<FVector::FReal>(Impulse.X, -MaxTouchImpulse, MaxTouchImpulse);
					Impulse.Y = FMath::Clamp<FVector::FReal>(Impulse.Y, -MaxTouchImpulse, MaxTouchImpulse);
					Impulse.Z = FMath::Clamp<FVector::FReal>(Impulse.Z, -MaxTouchImpulse, MaxTouchImpulse);
					FoliageForce += Impulse;
				}
			}
		}

		// Apply spring stiffness, which is the force that pushes the simulated particle back to the origin
		FoliageForce += -FoliageStiffness * FoliagePosition;
		// Apply spring quadratic stiffness, which increases in magnitude with the square of the distance to the origin
		// This prevents the spring from being displaced too much by touch and damage forces
		FoliageForce += -FoliageStiffnessQuadratic * FoliagePosition.SizeSquared() * FoliagePosition.GetSafeNormal();
		// Apply spring damping, which is like air resistance and causes the spring to lose energy over time
		FoliageForce += -FoliageDamping * FoliageVelocity;

		FoliageForce.X = FMath::Clamp<FVector::FReal>(FoliageForce.X, -MaxForce, MaxForce);
		FoliageForce.Y = FMath::Clamp<FVector::FReal>(FoliageForce.Y, -MaxForce, MaxForce);
		FoliageForce.Z = FMath::Clamp<FVector::FReal>(FoliageForce.Z, -MaxForce, MaxForce);

		FoliageVelocity += FoliageForce * DeltaSeconds;
		FoliageForce = FVector::ZeroVector;

		const float MaxVelocity = 1000.0f;
		FoliageVelocity.X = FMath::Clamp<FVector::FReal>(FoliageVelocity.X, -MaxVelocity, MaxVelocity);
		FoliageVelocity.Y = FMath::Clamp<FVector::FReal>(FoliageVelocity.Y, -MaxVelocity, MaxVelocity);
		FoliageVelocity.Z = FMath::Clamp<FVector::FReal>(FoliageVelocity.Z, -MaxVelocity, MaxVelocity);

		FoliagePosition += FoliageVelocity * DeltaSeconds;

		//DrawDebugLine(GetWorld(), GetRootComponent()->Bounds.Origin + FVector(0,0,100), GetRootComponent()->Bounds.Origin + FVector(0,0,100) + FoliagePosition, 255, 100, 100, false);

		//@todo - derive this height from the static mesh
		const float IntersectionHeight = 100.0f;
		// Calculate the rotation angle using Sin(Angle) = Opposite / Hypotenuse
		const FVector::FReal RotationAngle = -FMath::Asin(FoliagePosition.Size() / IntersectionHeight);
		// Use a rotation angle perpendicular to the impulse direction and the z axis
		const FVector NormalizedRotationAxis = FoliagePosition.SizeSquared() > KINDA_SMALL_NUMBER ? 
			(FoliagePosition ^ FVector(0,0,1)).GetSafeNormal() :
			FVector(0,0,1);

		// Propagate the new rotation axis and angle to the rendering thread
		FoliageComponent->FoliageSceneProxy->UpdateParameters_GameThread(FoliagePosition, FVector4(NormalizedRotationAxis, RotationAngle));

		if (FoliagePosition.SizeSquared() < FMath::Square(KINDA_SMALL_NUMBER * 10.0f)
			&& FoliageVelocity.SizeSquared() < FMath::Square(KINDA_SMALL_NUMBER * 10.0f))
		{
			// Go into stasis (will no longer be ticked) if this actor's spring simulation has stabilized
			SetActorTickEnabled(false);
		}
	}
	Super::Tick(DeltaSeconds);
}

void AInteractiveFoliageActor::PostActorCreated()
{
	Super::PostActorCreated();
	SetupCollisionCylinder();
}

void AInteractiveFoliageActor::PostLoad()
{
	Super::PostLoad();
	SetupCollisionCylinder();
}
