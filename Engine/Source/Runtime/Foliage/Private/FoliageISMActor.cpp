// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageISMActor.h"
#include "Engine/Blueprint.h"
#include "InstancedFoliageActor.h"
#include "FoliageType.h"
#include "FoliageType_Actor.h"
#include "FoliageHelper.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Components/StaticMeshComponent.h"
#include "ISMPartition/ISMComponentDescriptor.h"
#include "FoliageInstancedStaticMeshComponent.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Math/Box.h"

FFoliageISMActor::~FFoliageISMActor()
{
#if WITH_EDITOR
	UnregisterDelegates();
#endif
}

void FFoliageISMActor::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	Ar << Guid;
	ClientHandle.Serialize(Ar);
	Ar << ISMDefinition;
	Ar << ActorClass;
#endif
}

void FFoliageISMActor::PostSerialize(FArchive& Ar)
{
	FFoliageImpl::PostSerialize(Ar);
#if WITH_EDITOR
	if (GIsEditor && IsInitialized() && Ar.IsLoading())
	{
		GetIFA()->RegisterClientInstanceManager(ClientHandle, this);
	}
#endif
}

void FFoliageISMActor::PostLoad()
{
	FFoliageImpl::PostLoad();
#if WITH_EDITOR
	if (GIsEditor && IsInitialized())
	{
		RegisterDelegates();
	}
#endif
}

#if WITH_EDITOR
void FFoliageISMActor::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ActorClass, InThis);
}

bool FFoliageISMActor::IsInitialized() const
{
	return ClientHandle.IsValid();
}

void InitDescriptorFromFoliageType(FISMComponentDescriptor& Descriptor, const UFoliageType* FoliageType)
{
	const UFoliageType_Actor* FoliageTypeActor = Cast<UFoliageType_Actor>(FoliageType);
	Descriptor.ComponentClass = FoliageTypeActor->StaticMeshOnlyComponentClass != nullptr ? FoliageTypeActor->StaticMeshOnlyComponentClass.Get() : UFoliageInstancedStaticMeshComponent::StaticClass();
	
	Descriptor.Mobility = FoliageType->Mobility;
	Descriptor.InstanceStartCullDistance = FoliageType->CullDistance.Min;
	Descriptor.InstanceEndCullDistance = FoliageType->CullDistance.Max;
	Descriptor.bCastShadow = FoliageType->CastShadow;
	Descriptor.bCastDynamicShadow = FoliageType->bCastDynamicShadow;
	Descriptor.bCastStaticShadow = FoliageType->bCastStaticShadow;
	Descriptor.bCastContactShadow = FoliageType->bCastContactShadow;
	Descriptor.RuntimeVirtualTextures = FoliageType->RuntimeVirtualTextures;
	Descriptor.VirtualTextureRenderPassType = FoliageType->VirtualTextureRenderPassType;
	Descriptor.VirtualTextureCullMips = FoliageType->VirtualTextureCullMips;
	Descriptor.TranslucencySortPriority = FoliageType->TranslucencySortPriority;
	Descriptor.bAffectDynamicIndirectLighting = FoliageType->bAffectDynamicIndirectLighting;
	Descriptor.bAffectDistanceFieldLighting = FoliageType->bAffectDistanceFieldLighting;
	Descriptor.bCastShadowAsTwoSided = FoliageType->bCastShadowAsTwoSided;
	Descriptor.bReceivesDecals = FoliageType->bReceivesDecals;
	Descriptor.bOverrideLightMapRes = FoliageType->bOverrideLightMapRes;
	Descriptor.OverriddenLightMapRes = FoliageType->OverriddenLightMapRes;
	Descriptor.LightmapType = FoliageType->LightmapType;
	Descriptor.bUseAsOccluder = FoliageType->bUseAsOccluder;
	Descriptor.bEnableDensityScaling = FoliageType->bEnableDensityScaling;
	Descriptor.LightingChannels = FoliageType->LightingChannels;
	Descriptor.bRenderCustomDepth = FoliageType->bRenderCustomDepth;
	Descriptor.CustomDepthStencilWriteMask = FoliageType->CustomDepthStencilWriteMask;
	Descriptor.CustomDepthStencilValue = FoliageType->CustomDepthStencilValue;
	Descriptor.bIncludeInHLOD = FoliageType->bIncludeInHLOD;
	Descriptor.BodyInstance.CopyBodyInstancePropertiesFrom(&FoliageType->BodyInstance);

	Descriptor.bHasCustomNavigableGeometry = FoliageType->CustomNavigableGeometry;
	Descriptor.bEnableDiscardOnLoad = FoliageType->bEnableDiscardOnLoad;
	Descriptor.ShadowCacheInvalidationBehavior = FoliageType->ShadowCacheInvalidationBehavior;
}

void FFoliageISMActor::Initialize(const UFoliageType* FoliageType)
{
	check(!IsInitialized());

	AInstancedFoliageActor* IFA = GetIFA();

	const UFoliageType_Actor* FoliageTypeActor = Cast<const UFoliageType_Actor>(FoliageType);
	ActorClass = FoliageTypeActor->ActorClass ? FoliageTypeActor->ActorClass.Get() : AActor::StaticClass();
	
	FActorSpawnParameters SpawnParams;
	SpawnParams.bCreateActorPackage = false;
	SpawnParams.bNoFail = true;
	SpawnParams.bHideFromSceneOutliner = true;
	SpawnParams.bTemporaryEditorActor = true;
	SpawnParams.ObjectFlags &= ~RF_Transactional;
	SpawnParams.ObjectFlags |= RF_Transient;
	AActor* SpawnedActor = IFA->GetWorld()->SpawnActor<AActor>(ActorClass, SpawnParams);
	FTransform ActorTransform = SpawnedActor->GetActorTransform();
	TArray<UStaticMeshComponent*> StaticMeshComponents;
	SpawnedActor->GetComponents(StaticMeshComponents);

	ClientHandle = IFA->RegisterClient(Guid);
	IFA->RegisterClientInstanceManager(ClientHandle, this);

	for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
	{
		FISMComponentDescriptor Descriptor;
		// Avoid initializing the body instance as we are going to do it in the InitDescriptorFromFoliageType and that Copy of BodyInstance on a registered components will fail.
		Descriptor.InitFrom(StaticMeshComponent, /*bInitBodyInstance*/ false);
		InitDescriptorFromFoliageType(Descriptor, FoliageTypeActor);
		Descriptor.ComputeHash();

		int32 DescriptorIndex = IFA->RegisterISMComponentDescriptor(Descriptor);
		TArray<FTransform>& Transforms = ISMDefinition.FindOrAdd(DescriptorIndex);
		if (UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(StaticMeshComponent))
		{
			for (int32 InstanceIndex = 0; InstanceIndex < ISMComponent->GetInstanceCount(); ++InstanceIndex)
			{
				FTransform InstanceTransform;
				if (ensure(ISMComponent->GetInstanceTransform(InstanceIndex, InstanceTransform, /*bWorldSpace=*/ true)))
				{
					FTransform LocalTransform = InstanceTransform.GetRelativeTransform(ActorTransform);
					Transforms.Add(LocalTransform);
				}
			}
		}
		else
		{
			FTransform LocalTransform = StaticMeshComponent->GetComponentTransform().GetRelativeTransform(ActorTransform);
			Transforms.Add(LocalTransform);
		}
	}

	IFA->GetWorld()->DestroyActor(SpawnedActor);
	
	RegisterDelegates();
}

void FFoliageISMActor::Uninitialize()
{
	check(IsInitialized());
	UnregisterDelegates();
	GetIFA()->UnregisterClient(ClientHandle);
	ISMDefinition.Empty();
}

void FFoliageISMActor::RegisterDelegates()
{
	if (ActorClass)
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(ActorClass->ClassGeneratedBy))
		{
			Blueprint->OnCompiled().AddRaw(this, &FFoliageISMActor::OnBlueprintChanged);
		}
	}
}

void FFoliageISMActor::UnregisterDelegates()
{
	if (ActorClass)
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(ActorClass->ClassGeneratedBy))
		{
			Blueprint->OnCompiled().RemoveAll(this);
		}
	}
}

void FFoliageISMActor::OnBlueprintChanged(UBlueprint* InBlueprint)
{
	const UFoliageType_Actor* FoliageTypeActor = Cast<UFoliageType_Actor>(GetIFA()->GetFoliageTypeForInfo(Info));
	if (FoliageTypeActor)
	{
		Reapply(FoliageTypeActor);
	}
	else if (IsInitialized())
	{
		Uninitialize();
	}
}

void FFoliageISMActor::Reapply(const UFoliageType* FoliageType)
{
	if (IsInitialized())
	{
		{
			// We can't meaningfully re-instance the old static mesh instances to the new within this function, as)
			//	1) The old instances are destroyed prior to creating the new ones, so they are no longer available to map
			//	2) The new instances may not be a 1:1 mapping to the old, as they may now be a completely different structure
			// Instead we just re-instance the old static mesh instances to nothing, so that anything referencing them is cleaned-up
			TMap<FSMInstanceId, FSMInstanceId> ReplacementSMInstanceIds;
			GetIFA()->ForEachClientSMInstance(ClientHandle, [&ReplacementSMInstanceIds](FSMInstanceId OldSMInstanceId)
			{
				ReplacementSMInstanceIds.Add(OldSMInstanceId, FSMInstanceId());
				return true;
			});
			UEngineElementsLibrary::ReplaceEditorSMInstanceElementHandles(ReplacementSMInstanceIds);
		}

		Uninitialize();
	}
	Initialize(FoliageType);
	check(IsInitialized());
	
	BeginUpdate();
	for (const FFoliageInstance& Instance : Info->Instances)
	{
		AddInstance(Instance);
	}
	EndUpdate();
}

int32 FFoliageISMActor::GetInstanceCount() const
{
	return Info->Instances.Num();
}

void FFoliageISMActor::PreAddInstances(const UFoliageType* FoliageType, int32 AddedInstanceCount)
{
	if (!IsInitialized())
	{
		Initialize(FoliageType);
		check(IsInitialized());
	}

	GetIFA()->ReserveISMInstances(ClientHandle, AddedInstanceCount, ISMDefinition);
}

void FFoliageISMActor::AddInstance(const FFoliageInstance& NewInstance)
{
	GetIFA()->AddISMInstance(ClientHandle, NewInstance.GetInstanceWorldTransform(), ISMDefinition);
}

void FFoliageISMActor::RemoveInstance(int32 InstanceIndex)
{
	bool bOutIsEmpty = false;
	GetIFA()->RemoveISMInstance(ClientHandle, InstanceIndex, &bOutIsEmpty);
	
	if(bOutIsEmpty)
	{
		Uninitialize();
	}
}

void FFoliageISMActor::BeginUpdate()
{
	GetIFA()->BeginUpdate();
}

void FFoliageISMActor::EndUpdate()
{
	GetIFA()->EndUpdate();
}

void FFoliageISMActor::SetInstanceWorldTransform(int32 InstanceIndex, const FTransform& Transform, bool bTeleport)
{
	GetIFA()->SetISMInstanceTransform(ClientHandle, InstanceIndex, Transform, bTeleport, ISMDefinition);
}

FTransform FFoliageISMActor::GetInstanceWorldTransform(int32 InstanceIndex) const
{
	return Info->Instances[InstanceIndex].GetInstanceWorldTransform();
}

bool FFoliageISMActor::IsOwnedComponent(const UPrimitiveComponent* Component) const
{
	return GetIFA()->IsISMComponent(Component);
}

void FFoliageISMActor::Refresh(bool bAsync, bool bForce)
{
	GetIFA()->UpdateHISMTrees(bAsync, bForce);
}

void FFoliageISMActor::OnHiddenEditorViewMaskChanged(uint64 InHiddenEditorViews)
{
	if (!IsInitialized())
	{
		return;
	}

	// This can give weird results if toggling the visibility of 2 foliage types that share the same meshes. The last one wins for now.
	GetIFA()->ForEachClientComponent(ClientHandle, [InHiddenEditorViews](UInstancedStaticMeshComponent* Component)
	{
		if (UFoliageInstancedStaticMeshComponent* FoliageComponent = Cast<UFoliageInstancedStaticMeshComponent>(Component))
		{
			FoliageComponent->FoliageHiddenEditorViews = InHiddenEditorViews;
			FoliageComponent->MarkRenderStateDirty();
		}
		return true;
	});
}

void FFoliageISMActor::PreEditUndo(UFoliageType* FoliageType)
{
	FFoliageImpl::PreEditUndo(FoliageType);
	UnregisterDelegates();
}

void FFoliageISMActor::PostEditUndo(FFoliageInfo* InInfo, UFoliageType* FoliageType)
{
	FFoliageImpl::PostEditUndo(InInfo, FoliageType);
	RegisterDelegates();
}

void FFoliageISMActor::NotifyFoliageTypeWillChange(UFoliageType* FoliageType)
{
	UnregisterDelegates();
}

bool FFoliageISMActor::NotifyFoliageTypeChanged(UFoliageType* FoliageType, bool bSourceChanged)
{
	if (!IsInitialized())
	{
		return false;
	}
		
	if (UFoliageType_Actor* InFoliageTypeActor = Cast<UFoliageType_Actor>(FoliageType))
	{
		if (!InFoliageTypeActor->bStaticMeshOnly)
		{
			// requires implementation change
			return true;
		}
	}

	AInstancedFoliageActor* IFA = GetIFA();

	// Go through descriptors and see if they changed
	if (!bSourceChanged)
	{
		for (const auto& Pair : ISMDefinition)
		{
			const FISMComponentDescriptor& RegisteredDescriptor = IFA->GetISMComponentDescriptor(Pair.Key);
			FISMComponentDescriptor NewDescriptor(RegisteredDescriptor);
			InitDescriptorFromFoliageType(NewDescriptor, FoliageType);
			NewDescriptor.ComputeHash();

			if (RegisteredDescriptor != NewDescriptor)
			{
				bSourceChanged = true;
				break;
			}
		}
	}

	if (bSourceChanged)
	{
		Reapply(FoliageType);
		ApplySelection(true, Info->SelectedIndices);
	}
	else
	{
		RegisterDelegates();
	}

	return false;
}

void FFoliageISMActor::SelectAllInstances(bool bSelect)
{
	TSet<int32> Indices;
	Indices.Reserve(Info->Instances.Num());
	for (int32 i = 0; i < Info->Instances.Num(); ++i)
	{
		Indices.Add(i);
	}
	SelectInstances(bSelect, Indices);
}

void FFoliageISMActor::SelectInstance(bool bSelect, int32 Index)
{
	SelectInstances(bSelect, { Index });
}

void FFoliageISMActor::SelectInstances(bool bSelect, const TSet<int32>& SelectedIndices)
{
	GetIFA()->SelectISMInstances(ClientHandle, bSelect, SelectedIndices);
}

int32 FFoliageISMActor::GetInstanceIndexFrom(const UPrimitiveComponent* PrimitiveComponent, int32 ComponentIndex) const
{
	if (IsInitialized() && ComponentIndex != INDEX_NONE)
	{
		if (const UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(PrimitiveComponent))
		{
			return GetIFA()->GetISMInstanceIndex(ClientHandle, ISMComponent, ComponentIndex);
		}
	}
	return INDEX_NONE;
}

FBox FFoliageISMActor::GetSelectionBoundingBox(const TSet<int32>& SelectedIndices) const
{
	return GetIFA()->GetISMInstanceBounds(ClientHandle, SelectedIndices);
}
void FFoliageISMActor::ApplySelection(bool bApply, const TSet<int32>& SelectedIndices)
{
	// Going in and out of Folaige with an Empty/Unregistered impl.
	if (!IsInitialized())
	{
		return;
	}

	SelectAllInstances(false);
	if (bApply)
	{
		SelectInstances(true, SelectedIndices);
	}
}

void FFoliageISMActor::ClearSelection(const TSet<int32>& SelectedIndices)
{
	SelectAllInstances(false);
}

void FFoliageISMActor::ForEachSMInstance(TFunctionRef<bool(FSMInstanceId)> Callback) const
{
	if (ClientHandle)
	{
		GetIFA()->ForEachClientSMInstance(ClientHandle, Callback);
	}
}

void FFoliageISMActor::ForEachSMInstance(int32 InstanceIndex, TFunctionRef<bool(FSMInstanceId)> Callback) const
{
	if (ClientHandle)
	{
		GetIFA()->ForEachClientSMInstance(ClientHandle, InstanceIndex, Callback);
	}
}

#endif // WITH_EDITOR

FText FFoliageISMActor::GetISMPartitionInstanceDisplayName(const FISMClientInstanceId& InstanceId) const
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = ISMClientInstanceIdToFoliageInstanceId(InstanceId);
	const FText OwnerDisplayName = FText::FromString(ActorClass ? ActorClass->GetName() : GetIFA()->GetName());
	return FText::Format(NSLOCTEXT("FoliageISMActor", "DisplayNameFmt", "{0} - Instance {1}"), OwnerDisplayName, FoliageInstanceId.Index);
#else
	return FText();
#endif
}

FText FFoliageISMActor::GetISMPartitionInstanceTooltip(const FISMClientInstanceId& InstanceId) const
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = ISMClientInstanceIdToFoliageInstanceId(InstanceId);
	const FText OwnerDisplayPath = FText::FromString(GetIFA()->GetPathName(GetIFA()->GetWorld())); // stops the path at the level of the world the object is in
	return FText::Format(NSLOCTEXT("FoliageISMActor", "TooltipFmt", "Instance {0} on {1}"), FoliageInstanceId.Index, OwnerDisplayPath);
#else
	return FText();
#endif
}

bool FFoliageISMActor::CanEditISMPartitionInstance(const FISMClientInstanceId& InstanceId) const
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = ISMClientInstanceIdToFoliageInstanceId(InstanceId);
	return GetIFA()->CanEditFoliageInstance(FoliageInstanceId);
#else
	return false;
#endif
}

bool FFoliageISMActor::CanMoveISMPartitionInstance(const FISMClientInstanceId& InstanceId, const ETypedElementWorldType InWorldType) const
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = ISMClientInstanceIdToFoliageInstanceId(InstanceId);
	return GetIFA()->CanMoveFoliageInstance(FoliageInstanceId, InWorldType);
#else
	return false;
#endif
}

bool FFoliageISMActor::GetISMPartitionInstanceTransform(const FISMClientInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace) const
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = ISMClientInstanceIdToFoliageInstanceId(InstanceId);
	return GetIFA()->GetFoliageInstanceTransform(FoliageInstanceId, OutInstanceTransform, bWorldSpace);
#else
	return false;
#endif
}

bool FFoliageISMActor::SetISMPartitionInstanceTransform(const FISMClientInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace, bool bTeleport)
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = ISMClientInstanceIdToFoliageInstanceId(InstanceId);
	return GetIFA()->SetFoliageInstanceTransform(FoliageInstanceId, InstanceTransform, bWorldSpace, bTeleport);
#else
	return false;
#endif
}

void FFoliageISMActor::NotifyISMPartitionInstanceMovementStarted(const FISMClientInstanceId& InstanceId)
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = ISMClientInstanceIdToFoliageInstanceId(InstanceId);
	GetIFA()->NotifyFoliageInstanceMovementStarted(FoliageInstanceId);
#endif
}

void FFoliageISMActor::NotifyISMPartitionInstanceMovementOngoing(const FISMClientInstanceId& InstanceId)
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = ISMClientInstanceIdToFoliageInstanceId(InstanceId);
	GetIFA()->NotifyFoliageInstanceMovementOngoing(FoliageInstanceId);
#endif
}

void FFoliageISMActor::NotifyISMPartitionInstanceMovementEnded(const FISMClientInstanceId& InstanceId)
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = ISMClientInstanceIdToFoliageInstanceId(InstanceId);
	GetIFA()->NotifyFoliageInstanceMovementEnded(FoliageInstanceId);
#endif
}

void FFoliageISMActor::NotifyISMPartitionInstanceSelectionChanged(const FISMClientInstanceId& InstanceId, const bool bIsSelected)
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = ISMClientInstanceIdToFoliageInstanceId(InstanceId);
	GetIFA()->NotifyFoliageInstanceSelectionChanged(FoliageInstanceId, bIsSelected);
#endif
}

bool FFoliageISMActor::DeleteISMPartitionInstances(TArrayView<const FISMClientInstanceId> InstanceIds)
{
#if WITH_EDITOR
	const TArray<FFoliageInstanceId> FoliageInstanceIds = ISMClientInstanceIdsToFoliageInstanceIds(InstanceIds);
	return GetIFA()->DeleteFoliageInstances(FoliageInstanceIds);
#else
	return false;
#endif
}

bool FFoliageISMActor::DuplicateISMPartitionInstances(TArrayView<const FISMClientInstanceId> InstanceIds, TArray<FISMClientInstanceId>& OutNewInstanceIds)
{
#if WITH_EDITOR
	const TArray<FFoliageInstanceId> FoliageInstanceIds = ISMClientInstanceIdsToFoliageInstanceIds(InstanceIds);

	TArray<FFoliageInstanceId> NewFoliageInstanceIds;
	const bool bDidDuplicate = GetIFA()->DuplicateFoliageInstances(FoliageInstanceIds, NewFoliageInstanceIds);

	OutNewInstanceIds.Reset(NewFoliageInstanceIds.Num());
	for (const FFoliageInstanceId& NewFoliageInstanceId : NewFoliageInstanceIds)
	{
		OutNewInstanceIds.Add(FISMClientInstanceId{ static_cast<FFoliageISMActor&>(*NewFoliageInstanceId.Info->Implementation).ClientHandle, NewFoliageInstanceId.Index });
	}

	return bDidDuplicate;
#else
	return false;
#endif
}

#if WITH_EDITOR
FFoliageInstanceId FFoliageISMActor::ISMClientInstanceIdToFoliageInstanceId(const FISMClientInstanceId& InstanceId) const
{
	check(InstanceId.Handle == ClientHandle);
	return FFoliageInstanceId{ Info, InstanceId.Index };
}

TArray<FFoliageInstanceId> FFoliageISMActor::ISMClientInstanceIdsToFoliageInstanceIds(TArrayView<const FISMClientInstanceId> InstanceIds) const
{
	TArray<FFoliageInstanceId> FoliageInstanceIds;
	FoliageInstanceIds.Reserve(InstanceIds.Num());
	for (const FISMClientInstanceId& InstanceId : InstanceIds)
	{
		FoliageInstanceIds.Add(ISMClientInstanceIdToFoliageInstanceId(InstanceId));
	}
	return FoliageInstanceIds;
}
#endif
