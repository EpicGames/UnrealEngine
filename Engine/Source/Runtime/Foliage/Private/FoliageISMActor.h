// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "InstancedFoliage.h"
#include "ISMPartition/ISMPartitionActor.h"
#include "ISMPartition/ISMComponentDescriptor.h"
#include "Misc/Guid.h"
#include "Containers/SortedMap.h"

struct FFoliageInfo;
class UInstancedStaticMeshComponent;
class UPrimitiveComponent;
class UBlueprint;
class UFoliageType_Actor;

struct FFoliageISMActor : public FFoliageImpl, public IISMPartitionInstanceManager
{
	FFoliageISMActor(FFoliageInfo* Info)
		: FFoliageImpl(Info)
#if WITH_EDITORONLY_DATA
		, Guid(FGuid::NewGuid())
		, ActorClass(nullptr)
#endif
	{
	}

	virtual ~FFoliageISMActor();

#if WITH_EDITORONLY_DATA
	FGuid Guid;
	FISMClientHandle ClientHandle;
	TSortedMap<int32, TArray<FTransform>> ISMDefinition;
	TObjectPtr<UClass> ActorClass;
#endif
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostSerialize(FArchive& Ar) override;
	virtual void PostLoad() override;
		
#if WITH_EDITOR
	virtual void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector) override;
	virtual bool IsInitialized() const override;
	virtual void Initialize(const UFoliageType* FoliageType) override;
	virtual void Uninitialize() override;
	virtual void Reapply(const UFoliageType* FoliageType) override;
	virtual int32 GetInstanceCount() const override;
	virtual void PreAddInstances(const UFoliageType* FoliageType, int32 AddedInstanceCount) override;
	virtual void AddInstance(const FFoliageInstance& NewInstance) override;
	virtual void RemoveInstance(int32 InstanceIndex) override;
	virtual void SetInstanceWorldTransform(int32 InstanceIndex, const FTransform& Transform, bool bTeleport) override;
	virtual FTransform GetInstanceWorldTransform(int32 InstanceIndex) const override;
	virtual bool IsOwnedComponent(const UPrimitiveComponent* Component) const override;
	
	virtual void SelectAllInstances(bool bSelect) override;
	virtual void SelectInstance(bool bSelect, int32 Index) override;
	virtual void SelectInstances(bool bSelect, const TSet<int32>& SelectedIndices) override;
	virtual int32 GetInstanceIndexFrom(const UPrimitiveComponent* Component, int32 ComponentIndex) const;
	virtual FBox GetSelectionBoundingBox(const TSet<int32>& SelectedIndices) const override;
	virtual void ApplySelection(bool bApply, const TSet<int32>& SelectedIndices) override;
	virtual void ClearSelection(const TSet<int32>& SelectedIndices) override;

	virtual void ForEachSMInstance(TFunctionRef<bool(FSMInstanceId)> Callback) const override;
	virtual void ForEachSMInstance(int32 InstanceIndex, TFunctionRef<bool(FSMInstanceId)> Callback) const override;

	virtual void BeginUpdate() override;
	virtual void EndUpdate() override;
	virtual void Refresh(bool bAsync, bool bForce) override;
	virtual void OnHiddenEditorViewMaskChanged(uint64 InHiddenEditorViews) override;
	virtual void PreEditUndo(UFoliageType* FoliageType) override;
	virtual void PostEditUndo(FFoliageInfo* InInfo, UFoliageType* FoliageType) override;
	virtual void NotifyFoliageTypeWillChange(UFoliageType* FoliageType) override;
	virtual bool NotifyFoliageTypeChanged(UFoliageType* FoliageType, bool bSourceChanged) override;

private:
	void RegisterDelegates();
	void UnregisterDelegates();
	void OnBlueprintChanged(UBlueprint* InBlueprint);
#endif

private:
	//~ IISMPartitionInstanceManager interface
	virtual FText GetISMPartitionInstanceDisplayName(const FISMClientInstanceId& InstanceId) const override;
	virtual FText GetISMPartitionInstanceTooltip(const FISMClientInstanceId& InstanceId) const override;
	virtual bool CanEditISMPartitionInstance(const FISMClientInstanceId& InstanceId) const override;
	virtual bool CanMoveISMPartitionInstance(const FISMClientInstanceId& InstanceId, const ETypedElementWorldType InWorldType) const override;
	virtual bool GetISMPartitionInstanceTransform(const FISMClientInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace = false) const override;
	virtual bool SetISMPartitionInstanceTransform(const FISMClientInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace = false, bool bTeleport = false) override;
	virtual void NotifyISMPartitionInstanceMovementStarted(const FISMClientInstanceId& InstanceId) override;
	virtual void NotifyISMPartitionInstanceMovementOngoing(const FISMClientInstanceId& InstanceId) override;
	virtual void NotifyISMPartitionInstanceMovementEnded(const FISMClientInstanceId& InstanceId) override;
	virtual void NotifyISMPartitionInstanceSelectionChanged(const FISMClientInstanceId& InstanceId, const bool bIsSelected) override;
	virtual bool DeleteISMPartitionInstances(TArrayView<const FISMClientInstanceId> InstanceIds) override;
	virtual bool DuplicateISMPartitionInstances(TArrayView<const FISMClientInstanceId> InstanceIds, TArray<FISMClientInstanceId>& OutNewInstanceIds) override;

#if WITH_EDITOR
	FFoliageInstanceId ISMClientInstanceIdToFoliageInstanceId(const FISMClientInstanceId& InstanceId) const;
	TArray<FFoliageInstanceId> ISMClientInstanceIdsToFoliageInstanceIds(TArrayView<const FISMClientInstanceId> InstanceIds) const;
#endif
};
