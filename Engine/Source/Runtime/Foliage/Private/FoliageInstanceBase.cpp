// Copyright Epic Games, Inc. All Rights Reserved.
#include "FoliageInstanceBase.h"
#include "GameFramework/Actor.h"
#include "Misc/PackageName.h"
#include "InstancedFoliageActor.h"
#include "InstancedFoliage.h"
#include "Engine/Level.h"
#include "Engine/WorldComposition.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Modules/ModuleManager.h"
#include "FoliageModule.h"

#if WITH_EDITORONLY_DATA

FFoliageInstanceBaseInfo::FFoliageInstanceBaseInfo()
	: CachedLocation(FVector::ZeroVector)
	, CachedRotation(FRotator::ZeroRotator)
	, CachedDrawScale(1.0f, 1.0f, 1.0f)
{}

FFoliageInstanceBaseInfo::FFoliageInstanceBaseInfo(UActorComponent* InComponent)
	: BasePtr(InComponent)
	, CachedLocation(FVector::ZeroVector)
	, CachedRotation(FRotator::ZeroRotator)
	, CachedDrawScale(1.0f, 1.0f, 1.0f)
{
	UpdateLocationFromComponent(InComponent);
}

void FFoliageInstanceBaseInfo::UpdateLocationFromComponent(UActorComponent* InComponent)
{
	if (InComponent)
	{
		AActor* Owner = Cast<AActor>(InComponent->GetOuter());
		if (Owner)
		{
			const USceneComponent* RootComponent = Owner->GetRootComponent();
			if (RootComponent)
			{
				CachedLocation = RootComponent->GetRelativeLocation();
				CachedRotation = RootComponent->GetRelativeRotation();
				CachedDrawScale = RootComponent->GetRelativeScale3D();
			}
		}
	}
}

FArchive& operator << (FArchive& Ar, FFoliageInstanceBaseInfo& BaseInfo)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::FoliageLazyObjPtrToSoftObjPtr)
	{
		Ar << BaseInfo.BasePtr_DEPRECATED;
	}
	else
	{
		// This is a cached reference, we don't want to warn about the target actor being deleted
		FSoftObjectPathSerializationScope SerializationScope(NAME_None, NAME_None, ESoftObjectPathCollectType::NeverCollect, ESoftObjectPathSerializeType::AlwaysSerialize);
		Ar << BaseInfo.BasePtr;
	}

	Ar << BaseInfo.CachedLocation;
	Ar << BaseInfo.CachedRotation;
	Ar << BaseInfo.CachedDrawScale;

	return Ar;
}

//////////////////////////////////////////////////////////////////////////
//
//////////////////////////////////////////////////////////////////////////
FFoliageInstanceBaseId FFoliageInstanceBaseCache::InvalidBaseId = INDEX_NONE;

FFoliageInstanceBaseCache::FFoliageInstanceBaseCache()
	: NextBaseId(1)
{
}

FArchive& operator << (FArchive& Ar, FFoliageInstanceBaseCache& InstanceBaseCache)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Ar << InstanceBaseCache.NextBaseId;
	Ar << InstanceBaseCache.InstanceBaseMap;

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::FoliageLazyObjPtrToSoftObjPtr)
	{
		Ar << InstanceBaseCache.InstanceBaseLevelMap_DEPRECATED;		
	}
	else
	{
		// This is a cached reference, we don't want to warn about the target actor being deleted
		FSoftObjectPathSerializationScope SerializationScope(NAME_None, NAME_None, ESoftObjectPathCollectType::NeverCollect, ESoftObjectPathSerializeType::AlwaysSerialize);

		Ar << InstanceBaseCache.InstanceBaseLevelMap;

		if (Ar.IsTransacting())
		{
			Ar << InstanceBaseCache.InstanceBaseInvMap;
		}
		else if (Ar.IsLoading()) // Regenerate inverse map whenever we load cache
		{
			InstanceBaseCache.InstanceBaseInvMap.Empty();
			for (const auto& Pair : InstanceBaseCache.InstanceBaseMap)
			{
				const FFoliageInstanceBaseInfo& BaseInfo = Pair.Value;
				if (InstanceBaseCache.InstanceBaseInvMap.Contains(BaseInfo.BasePtr))
				{
					// more info for UE-30878
					UE_LOGF(LogInstancedFoliage, Warning, "Instance base cache - integrity verification(3): Counter: %d Size: %d, InvSize: %d (Key: %ls)",
						(int32)InstanceBaseCache.NextBaseId, InstanceBaseCache.InstanceBaseMap.Num(), InstanceBaseCache.InstanceBaseInvMap.Num(),
						*BaseInfo.BasePtr.GetUniqueID().ToString());
				}
				else
				{
					InstanceBaseCache.InstanceBaseInvMap.Add(BaseInfo.BasePtr, Pair.Key);
				}
			}
		}
	}	

	return Ar;
}

FFoliageInstanceBaseId FFoliageInstanceBaseCache::AddInstanceBaseId(UActorComponent* InComponent)
{
	FFoliageInstanceBaseId BaseId = FFoliageInstanceBaseCache::InvalidBaseId;
	bool bIgnoreComponentClass = false; 
#if WITH_EDITOR
	if (IFoliageEditModuleBase* FoliageModule = IFoliageEditModuleBase::Get(); ensure(FoliageModule))
	{
		bIgnoreComponentClass = FoliageModule->ShouldIgnoreComponentForBaseID(InComponent);
	}
#endif // WITH_EDITOR

	if (InComponent && !InComponent->IsCreatedByConstructionScript() && !bIgnoreComponentClass)
	{
		BaseId = GetInstanceBaseId(InComponent);
		FFoliageInstanceBaseInfo BaseInfo(InComponent);
		if (BaseId == FFoliageInstanceBaseCache::InvalidBaseId)
		{
			// generate next unique ID for base component
			do
			{
				BaseId = NextBaseId++;
			}
			while (InstanceBaseMap.Contains(BaseId));
			
			// more info for UE-30878
			if (InstanceBaseInvMap.Contains(BaseInfo.BasePtr))
			{
				FSoftObjectPath BaseUID = BaseInfo.BasePtr.GetUniqueID();
				UE_LOGF(LogInstancedFoliage, Error, "Instance base cache - integrity verification(2): Counter: %d Size: %d, InvSize: %d, BaseUID: %ls, BaseName: %ls", 
					(int32)BaseId, InstanceBaseMap.Num(), InstanceBaseInvMap.Num(), *BaseUID.ToString(), *InComponent->GetFullName());
			}
						
			InstanceBaseMap.Add(BaseId, BaseInfo);
			InstanceBaseInvMap.Add(BaseInfo.BasePtr, BaseId);

			ULevel* ComponentLevel = InComponent->GetComponentLevel();
			if (ComponentLevel)
			{
				UWorld* ComponentWorld = Cast<UWorld>(ComponentLevel->GetOuter());
				if (ComponentWorld)
				{
					auto WorldKey = TSoftObjectPtr<UWorld>(ComponentWorld);
					InstanceBaseLevelMap.FindOrAdd(WorldKey).Add(BaseInfo.BasePtr);
				}
			}
		}
		else
		{
			// Update BaseInfo
			InstanceBaseMap[BaseId] = MoveTemp(BaseInfo);
		}
	}

	return BaseId;
}

FFoliageInstanceBaseId FFoliageInstanceBaseCache::GetInstanceBaseId(UActorComponent* InComponent) const
{
	if(InComponent && (InstanceBaseInvMap.Num() > 0))
	{
		FFoliageInstanceBasePtr BasePtr(InComponent);
		if (BasePtr.IsValid())
		{
			return GetInstanceBaseId(BasePtr);
		}
	}

	return InvalidBaseId;
}


FFoliageInstanceBaseId FFoliageInstanceBaseCache::GetInstanceBaseId(const FFoliageInstanceBasePtr& BasePtr) const
{
	const FFoliageInstanceBaseId* BaseId = InstanceBaseInvMap.Find(BasePtr);
	if (!BaseId)
	{
		return FFoliageInstanceBaseCache::InvalidBaseId;
	}
	
	return *BaseId;
}

FFoliageInstanceBasePtr FFoliageInstanceBaseCache::GetInstanceBasePtr(FFoliageInstanceBaseId BaseId) const
{
	return GetInstanceBaseInfo(BaseId).BasePtr;
}

FFoliageInstanceBaseInfo FFoliageInstanceBaseCache::GetInstanceBaseInfo(FFoliageInstanceBaseId BaseId) const
{
	return InstanceBaseMap.FindRef(BaseId);
}

FFoliageInstanceBaseInfo FFoliageInstanceBaseCache::UpdateInstanceBaseInfoTransform(UActorComponent* InComponent)
{
	auto BaseId = GetInstanceBaseId(InComponent);
	if (BaseId != FFoliageInstanceBaseCache::InvalidBaseId)
	{
		auto* BaseInfo = InstanceBaseMap.Find(BaseId);
		check(BaseInfo);
		BaseInfo->UpdateLocationFromComponent(InComponent);
		return *BaseInfo;
	}

	return FFoliageInstanceBaseInfo();
}

void FFoliageInstanceBaseCache::CompactInstanceBaseCache(AInstancedFoliageActor* IFA)
{
	FFoliageInstanceBaseCache& Cache = IFA->InstanceBaseCache;

	UWorld* World = IFA->GetWorld();
	if (World != nullptr)
	{
		TSet<FFoliageInstanceBaseId> BasesInUse;
		for (auto& FoliagePair : IFA->FoliageInfos)
		{
			for (const auto& Pair : FoliagePair.Value->ComponentHash)
			{
				if (Pair.Key != FFoliageInstanceBaseCache::InvalidBaseId)
				{
					BasesInUse.Add(Pair.Key);
				}
			}
		}

		// Look for any removed maps
		TSet<FFoliageInstanceBasePtr> InvalidBasePtrs;
		for (auto Iter = Cache.InstanceBaseLevelMap.CreateIterator(); Iter; ++Iter)
		{
			TPair<TSoftObjectPtr<UWorld>, TArray<FFoliageInstanceBasePtr>>& Pair = *Iter;

			const auto& WorldAsset = Pair.Key;

			bool bExists = (WorldAsset == World);
			// Check sub-levels
			if (!bExists)
			{
				const FName PackageName = FName(*FPackageName::ObjectPathToPackageName(WorldAsset.ToString()));
				if (World->WorldComposition)
				{
					bExists = World->WorldComposition->DoesTileExists(PackageName);
				}
				else
				{
					bExists = (World->GetLevelStreamingForPackageName(PackageName) != nullptr);
				}
			}

			if (!bExists)
			{
				InvalidBasePtrs.Append(Pair.Value);
				Iter.RemoveCurrent();
			}
			else
			{
				// Remove dead links
				for (int32 i = Pair.Value.Num() - 1; i >= 0; --i)
				{
					// Base needs to be removed if it's not in use by existing instances or component was removed
					if (Pair.Value[i].IsNull() || !BasesInUse.Contains(Cache.GetInstanceBaseId(Pair.Value[i])))
					{
						InvalidBasePtrs.Add(Pair.Value[i]);
						Pair.Value.RemoveAt(i);
					}
				}

				if (Pair.Value.Num() == 0)
				{
					Iter.RemoveCurrent();
				}
			}
		}

		TSet<FFoliageInstanceBaseId> InvalidBaseIds;
		Cache.InstanceBaseInvMap.Empty();
		// Look for any removed base components
		for (auto Iter = Cache.InstanceBaseMap.CreateIterator(); Iter; ++Iter)
		{
			const TPair<FFoliageInstanceBaseId, FFoliageInstanceBaseInfo>& Pair = *Iter;

			const FFoliageInstanceBaseInfo& BaseInfo = Pair.Value;
			if (InvalidBasePtrs.Contains(BaseInfo.BasePtr))
			{
				InvalidBaseIds.Add(Pair.Key);
				Iter.RemoveCurrent();
			}
			else
			{
				// Regenerate inverse map
				Cache.InstanceBaseInvMap.Add(BaseInfo.BasePtr, Pair.Key);
			}
		}

		if (InvalidBaseIds.Num())
		{
			for (auto& Pair : IFA->FoliageInfos)
			{
				auto& MeshInfo = Pair.Value;
				MeshInfo->ComponentHash.Empty();
				int32 InstanceIdx = 0;

				for (FFoliageInstance& Instance : MeshInfo->Instances)
				{
					if (InvalidBaseIds.Contains(Instance.BaseId))
					{
						Instance.BaseId = FFoliageInstanceBaseCache::InvalidBaseId;
					}

					MeshInfo->ComponentHash.FindOrAdd(Instance.BaseId).Add(InstanceIdx);
					InstanceIdx++;
				}
			}

			Cache.InstanceBaseMap.Compact();
			Cache.InstanceBaseLevelMap.Compact();
		}
	}
}

void FFoliageInstanceBaseCache::UpdateInstanceBaseCachedTransforms()
{
	for (auto& Pair : InstanceBaseMap)
	{
		FFoliageInstanceBaseInfo& BaseInfo = Pair.Value;
		BaseInfo.UpdateLocationFromComponent(BaseInfo.BasePtr.Get());
	}
}

#endif//WITH_EDITORONLY_DATA
