// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageModule.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE( IFoliageModule, Foliage );

#if WITH_EDITOR
IFoliageEditModuleBase* IFoliageEditModuleBase::FoliageEditModulePtr = nullptr;

IFoliageEditModuleBase* IFoliageEditModuleBase::Get()
{
	return FoliageEditModulePtr;
}

void IFoliageEditModuleBase::SetFoliageEditModulePtr(IFoliageEditModuleBase* InFoliageEditModulePtr)
{
	FoliageEditModulePtr = InFoliageEditModulePtr;
}
#endif // WITH_EDITOR