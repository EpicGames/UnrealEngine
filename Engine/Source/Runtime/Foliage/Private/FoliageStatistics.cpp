// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageStatistics.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "EngineUtils.h"
#include "InstancedFoliageActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FoliageStatistics)

//////////////////////////////////////////////////////////////////////////
// UFoliageStatics

UFoliageStatistics::UFoliageStatistics(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UFoliageStatistics::FoliageOverlappingSphereCount(UObject* WorldContextObject, const UStaticMesh* StaticMesh, FVector CenterPosition, float Radius)
{
	int32 Count = 0;
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		const UObject* Source = Cast<UObject>(StaticMesh);
		const FSphere Sphere(CenterPosition, Radius);

		for (TActorIterator<AInstancedFoliageActor> It(World); It; ++It)
		{
			AInstancedFoliageActor* IFA = *It;
			if (IsValid(IFA))
			{
				TArray<const UFoliageType*> FoliageTypes;
				IFA->GetAllFoliageTypesForSource(Source, FoliageTypes);

				for (const auto Type : FoliageTypes)
				{
					Count += IFA->GetOverlappingSphereCount(Type, Sphere);
				}
			}
		}
	}

	return Count;
}

int32 UFoliageStatistics::FoliageOverlappingBoxCount(UObject* WorldContextObject, const UStaticMesh* StaticMesh, FBox Box)
{
	int32 Count = 0;
	const UObject* Source = Cast<UObject>(StaticMesh);
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		for (TActorIterator<AInstancedFoliageActor> It(World); It; ++It)
		{
			AInstancedFoliageActor* IFA = *It;
			if (IsValid(IFA))
			{
				TArray<const UFoliageType*> FoliageTypes;
				IFA->GetAllFoliageTypesForSource(Source, FoliageTypes);

				for (const auto Type : FoliageTypes)
				{
					Count += IFA->GetOverlappingBoxCount(Type, Box);
				}
			}
		}
	}

	return Count;
}

void UFoliageStatistics::FoliageOverlappingBoxTransforms(UObject* WorldContextObject, const UStaticMesh* StaticMesh, FBox Box, TArray<FTransform>& OutTransforms)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		const UObject* Source = Cast<UObject>(StaticMesh);
		for (TActorIterator<AInstancedFoliageActor> It(World); It; ++It)
		{
			AInstancedFoliageActor* IFA = *It;
			if (IsValid(IFA))
			{
				TArray<const UFoliageType*> FoliageTypes;
				IFA->GetAllFoliageTypesForSource(Source, FoliageTypes);

				for (const auto Type : FoliageTypes)
				{
					IFA->GetOverlappingBoxTransforms(Type, Box, OutTransforms);
				}
			}
		}
	}
}

void UFoliageStatistics::FoliageOverlappingMeshCounts_Debug(UObject* WorldContextObject, FVector CenterPosition, float Radius, TMap<UStaticMesh*, int32>& OutMeshCounts)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		const FSphere Sphere(CenterPosition, Radius);

		OutMeshCounts.Reset();
		for (TActorIterator<AInstancedFoliageActor> It(World); It; ++It)
		{
			AInstancedFoliageActor* IFA = *It;
			if (IsValid(IFA))
			{
				IFA->GetOverlappingMeshCounts(Sphere, OutMeshCounts);
			}
		}
	}
}

