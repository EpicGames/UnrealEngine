// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoliageTypeObject.h"
#include "Engine/Blueprint.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FoliageTypeObject)

void FFoliageTypeObject::RefreshInstance()
{
	//@todo: should verify that the instance is dirty before updating
	TypeInstance = nullptr;

	// If the foliage type is an asset, use that
	if (auto FoliageTypeAsAsset = Cast<UFoliageType>(FoliageTypeObject))
	{
		bIsAsset = true;
		TypeInstance = FoliageTypeAsAsset;
	}
	else if (auto FoliageTypeAsBP = Cast<UBlueprint>(FoliageTypeObject))
	{
		// If the foliage type is a BP class, create and store an instance of that class
		if (FoliageTypeAsBP->ParentClass->IsChildOf<UFoliageType_InstancedStaticMesh>())
		{
			bIsAsset = false;
			TypeInstance = NewObject<UFoliageType_InstancedStaticMesh>(GetTransientPackage(), *FoliageTypeAsBP->GeneratedClass, NAME_None, RF_Transactional);
		}
	}
}

const UFoliageType* FFoliageTypeObject::GetInstance()
{
	if (!TypeInstance)
	{
		// If the type instance doesn't exist, try to create it
		RefreshInstance();
	}

	return TypeInstance;
}

const UFoliageType* FFoliageTypeObject::GetInstance() const
{
	return TypeInstance;
}

bool FFoliageTypeObject::ContainsValidInstance() const
{
	return TypeInstance != nullptr;
}

bool FFoliageTypeObject::HasFoliageType() const
{
	return FoliageTypeObject != nullptr;
}

bool FFoliageTypeObject::IsDirty() const
{
	//@todo: make sure this works when the object isn't an asset
	return (!TypeInstance || TypeInstance->ChangeCount != TypeInstance->GetClass()->GetDefaultObject<UFoliageType>()->ChangeCount);
}

void FFoliageTypeObject::SetClean()
{
	if (TypeInstance)
	{
		TypeInstance->ChangeCount = TypeInstance->GetClass()->GetDefaultObject<UFoliageType>()->ChangeCount;
	}
}

#if WITH_EDITORONLY_DATA
void FFoliageTypeObject::PostSerialize(const FArchive& Ar)
{
	if (Ar.UEVer() < VER_UE4_FOLIAGE_WITH_ASSET_OR_CLASS)
	{
		// Set the type object to be the previously set FoliageType BP class
		if (Type_DEPRECATED)
		{
			FoliageTypeObject = Type_DEPRECATED->ClassGeneratedBy;
		}
	}
}
#endif
