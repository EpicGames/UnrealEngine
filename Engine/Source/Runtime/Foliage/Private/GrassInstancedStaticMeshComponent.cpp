// Copyright Epic Games, Inc. All Rights Reserved.
#include "GrassInstancedStaticMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GrassInstancedStaticMeshComponent)


static TAutoConsoleVariable<bool> CVarEnableGrassInstancedWPOVelocity(
	TEXT("r.Velocity.EnableLandscapeGrass"),
	true,
	TEXT("Specify if you want to output velocity for the grass component for WPO.\n")
	TEXT(" True (default)\n")
	TEXT(" False")
	);

bool UGrassInstancedStaticMeshComponent::SupportsWorldPositionOffsetVelocity() const 
{ 
	return CVarEnableGrassInstancedWPOVelocity.GetValueOnAnyThread() && Super::SupportsWorldPositionOffsetVelocity();
}
