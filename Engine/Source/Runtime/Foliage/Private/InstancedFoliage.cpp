// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
InstancedFoliage.cpp: Instanced foliage implementation.
=============================================================================*/

#include "InstancedFoliage.h"
#include "Templates/SubclassOf.h"
#include "HAL/IConsoleManager.h"
#include "GameFramework/DamageType.h"
#include "Engine/EngineTypes.h"
#include "Components/SceneComponent.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "CollisionQueryParams.h"
#include "WorldCollision.h"
#include "Engine/Blueprint.h"
#include "Engine/World.h"
#include "Components/PrimitiveComponent.h"
#include "FoliageType.h"
#include "UObject/UObjectIterator.h"
#include "FoliageInstancedStaticMeshComponent.h"
#include "FoliageType_InstancedStaticMesh.h"
#include "FoliageType_Actor.h"
#include "InstancedFoliageActor.h"
#include "Serialization/CustomVersion.h"
#include "UObject/Package.h"
#include "UObject/PropertyPortFlags.h"
#include "Engine/CollisionProfile.h"
#include "Engine/Brush.h"
#include "Engine/Engine.h"
#include "Components/BrushComponent.h"
#include "Components/ModelComponent.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "ProceduralFoliageComponent.h"
#include "ProceduralFoliageBlockingVolume.h"
#include "ProceduralFoliageVolume.h"
#include "EngineUtils.h"
#include "EngineGlobals.h"
#include "Engine/StaticMesh.h"
#include "DrawDebugHelpers.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/UObjectIterator.h"
#include "PreviewScene.h"
#include "FoliageActor.h"
#include "FoliageISMActor.h"
#include "LevelUtils.h"
#include "FoliageHelper.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/SMInstance/SMInstanceManager.h"
#include "Elements/SMInstance/SMInstanceElementData.h" // For SMInstanceElementDataUtil::SMInstanceElementsEnabled
#include "Algo/Transform.h"
#include "ActorPartition/ActorPartitionSubsystem.h"
#include "Misc/CoreMisc.h"
#include "Engine/DamageEvents.h"

#if WITH_EDITOR
#include "ActorEditorContext/ScopedActorEditorContextFromActor.h"
#endif

#define LOCTEXT_NAMESPACE "InstancedFoliage"

#define DO_FOLIAGE_CHECK			0			// whether to validate foliage data during editing.
#define FOLIAGE_CHECK_TRANSFORM		0			// whether to compare transforms between render and painting data.

DEFINE_LOG_CATEGORY(LogInstancedFoliage);

DECLARE_CYCLE_STAT(TEXT("FoliageActor_Trace"), STAT_FoliageTrace, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("FoliageMeshInfo_AddInstance"), STAT_FoliageAddInstance, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("FoliageMeshInfo_RemoveInstance"), STAT_FoliageRemoveInstance, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("FoliageMeshInfo_CreateComponent"), STAT_FoliageCreateComponent, STATGROUP_Foliage);

static TAutoConsoleVariable<int32> CVarFoliageDiscardDataOnLoad(
	TEXT("foliage.DiscardDataOnLoad"),
	0,
	TEXT("1: Discard foliage data on load if the foliage type has it enabled; 0: Keep foliage data regardless of whether the foliage type has it enabled or not (requires reloading level)"),
	ECVF_Scalability);

static TAutoConsoleVariable<float> CVarFoliageCullDistanceScale(
	TEXT("foliage.CullDistanceScale"),
	1.0,
	TEXT("Controls the cull distance scale. Foliage must opt-in to cull distance scaling through the foliage type."),
	ECVF_Scalability);

const FGuid FFoliageCustomVersion::GUID(0x430C4D19, 0x71544970, 0x87699B69, 0xDF90B0E5);
// Register the custom version with core
FCustomVersionRegistration GRegisterFoliageCustomVersion(FFoliageCustomVersion::GUID, FFoliageCustomVersion::LatestVersion, TEXT("FoliageVer"));

namespace FoliageElementUtil
{

static int32 GEnableFoliageInstanceElements = 0;
static FAutoConsoleVariableRef CVarEnableFoliageInstanceElements(
	TEXT("TypedElements.EnableFoliageInstanceElements"),
	GEnableFoliageInstanceElements,
	TEXT("Is support for static mesh instance elements enabled for foliage owned instances?")
	);

bool FoliageInstanceElementsEnabled()
{
	return SMInstanceElementDataUtil::SMInstanceElementsEnabled() && GEnableFoliageInstanceElements != 0;
}

}

///
/// FFoliageStaticMesh
///
struct FFoliageStaticMesh : public FFoliageImpl, public ISMInstanceManager
{
	FFoliageStaticMesh(FFoliageInfo* Info, UHierarchicalInstancedStaticMeshComponent* InComponent)
		: FFoliageImpl(Info)
		, Component(InComponent)
#if WITH_EDITOR
		, UpdateDepth(0)
		, bPreviousValue(false)
		, bInvalidateLightingCache(false)
#endif
	{

	}

	virtual ~FFoliageStaticMesh();

	TObjectPtr<UHierarchicalInstancedStaticMeshComponent> Component;
#if WITH_EDITOR
	int32 UpdateDepth;
	bool bPreviousValue;

	bool bInvalidateLightingCache;
#endif

	virtual void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector) override;
	virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR
	virtual bool IsInitialized() const override { return Component != nullptr; }
	virtual void Initialize(const UFoliageType* FoliageType) override;
	virtual void Uninitialize() override;
	virtual void Reapply(const UFoliageType* FoliageType) override;
	virtual int32 GetInstanceCount() const override;
	virtual void PreAddInstances(const UFoliageType* FoliageType, int32 Count) override;
	virtual void AddInstance(const FFoliageInstance& NewInstance) override;
	virtual void RemoveInstance(int32 InstanceIndex) override;
	virtual void SetInstanceWorldTransform(int32 InstanceIndex, const FTransform& Transform, bool bTeleport) override;
	virtual FTransform GetInstanceWorldTransform(int32 InstanceIndex) const override;
	virtual void PostUpdateInstances() override;
	virtual bool IsOwnedComponent(const UPrimitiveComponent* PrimitiveComponent) const override;

	virtual void SelectAllInstances(bool bSelect) override;
	virtual void SelectInstance(bool bSelect, int32 Index) override;
	virtual void SelectInstances(bool bSelect, const TSet<int32>& SelectedIndices) override;
	virtual int32 GetInstanceIndexFrom(const UPrimitiveComponent* PrimitiveComponent, int32 ComponentIndex) const override;
	virtual FBox GetSelectionBoundingBox(const TSet<int32>& SelectedIndices) const override;
	virtual void ApplySelection(bool bApply, const TSet<int32>& SelectedIndices) override;
	virtual void ClearSelection(const TSet<int32>& SelectedIndices) override;

	virtual void ForEachSMInstance(TFunctionRef<bool(FSMInstanceId)> Callback) const override;
	virtual void ForEachSMInstance(int32 InstanceIndex, TFunctionRef<bool(FSMInstanceId)> Callback) const override;

	virtual void BeginUpdate() override;
	virtual void EndUpdate() override;
	virtual void Refresh(bool Async, bool Force) override;
	virtual void OnHiddenEditorViewMaskChanged(uint64 InHiddenEditorViews) override;
	virtual void PreEditUndo(UFoliageType* FoliageType) override;
	virtual void PostEditUndo(FFoliageInfo* InInfo, UFoliageType* FoliageType) override;
	virtual void NotifyFoliageTypeWillChange(UFoliageType* FoliageType) override;
	virtual bool NotifyFoliageTypeChanged(UFoliageType* FoliageType, bool bSourceChanged) override;
	virtual void EnterEditMode() override;
	virtual void ExitEditMode() override;

	void HandleComponentMeshBoundsChanged(const FBoxSphereBounds& NewBounds);
#endif

	virtual int32 GetOverlappingSphereCount(const FSphere& Sphere) const override;
	virtual int32 GetOverlappingBoxCount(const FBox& Box) const override;
	virtual void GetOverlappingBoxTransforms(const FBox& Box, TArray<FTransform>& OutTransforms) const override;
	virtual void GetOverlappingMeshCount(const FSphere& Sphere, TMap<UStaticMesh*, int32>& OutCounts) const override;

	void UpdateComponentSettings(const UFoliageType_InstancedStaticMesh* InSettings);
	// Recreate the component if the FoliageType's ComponentClass doesn't match the Component's class
	void CheckComponentClass(const UFoliageType_InstancedStaticMesh* InSettings);
	void CreateNewComponent(const UFoliageType* InSettings);

private:
	//~ ISMInstanceManager interface
	virtual bool CanEditSMInstance(const FSMInstanceId& InstanceId) const override;
	virtual bool CanMoveSMInstance(const FSMInstanceId& InstanceId, const ETypedElementWorldType WorldType) const override;
	virtual bool GetSMInstanceTransform(const FSMInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace = false) const override;
	virtual bool SetSMInstanceTransform(const FSMInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace = false, bool bMarkRenderStateDirty = false, bool bTeleport = false) override;
	virtual void NotifySMInstanceMovementStarted(const FSMInstanceId& InstanceId) override;
	virtual void NotifySMInstanceMovementOngoing(const FSMInstanceId& InstanceId) override;
	virtual void NotifySMInstanceMovementEnded(const FSMInstanceId& InstanceId) override;
	virtual void NotifySMInstanceSelectionChanged(const FSMInstanceId& InstanceId, const bool bIsSelected) override;
	virtual bool DeleteSMInstances(TArrayView<const FSMInstanceId> InstanceIds) override;
	virtual bool DuplicateSMInstances(TArrayView<const FSMInstanceId> InstanceIds, TArray<FSMInstanceId>& OutNewInstanceIds) override;

#if WITH_EDITOR
	FFoliageInstanceId SMInstanceIdToFoliageInstanceId(const FSMInstanceId& InstanceId) const;
	TArray<FFoliageInstanceId> SMInstanceIdsToFoliageInstanceIds(TArrayView<const FSMInstanceId> InstanceIds) const;
#endif
};

namespace FoliageUtil
{
	FInt32Interval GetCullDistance(const UFoliageType* FoliageType, float CullDistanceScale)
	{
		if (!FoliageType->bEnableCullDistanceScaling)
		{
			return FoliageType->CullDistance;
		}

		return FInt32Interval(static_cast<int32>(FoliageType->CullDistance.Min * CullDistanceScale), static_cast<int32>(FoliageType->CullDistance.Max * CullDistanceScale));
	}

	void UpdateComponentCullDistance(const UFoliageType* FoliageType, UHierarchicalInstancedStaticMeshComponent* Component, float CullDistanceScale)
	{
		const FInt32Interval ScaledCullDistance = GetCullDistance(FoliageType, CullDistanceScale);

		Component->SetCullDistances(ScaledCullDistance.Min, ScaledCullDistance.Max);
	}

#if WITH_EDITOR
	static void CVarSinkFunction()
	{
		static float CachedCullDistanceScale = 1.0f;
		float CullDistanceScale = CVarFoliageCullDistanceScale.GetValueOnGameThread();

		if (CullDistanceScale != CachedCullDistanceScale)
		{
			CachedCullDistanceScale = CullDistanceScale;
			CullDistanceScale = FMath::Clamp(CullDistanceScale, 0.0f, 1.0f);

			for (const AInstancedFoliageActor* IFA : TObjectRange<AInstancedFoliageActor>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
			{
				for (const TPair<UFoliageType*, TUniqueObj<FFoliageInfo>>& Pair : IFA->GetFoliageInfos())
				{
					if (Pair.Key && Pair.Key->bEnableCullDistanceScaling)
					{
						if (Pair.Value->Type == EFoliageImplType::StaticMesh)
						{
							FFoliageStaticMesh* FoliageStaticMesh = StaticCast<FFoliageStaticMesh*>(Pair.Value->Implementation.Get());

							if (FoliageStaticMesh->Component != nullptr)
							{
								FoliageUtil::UpdateComponentCullDistance(Pair.Key, FoliageStaticMesh->Component, CullDistanceScale);
							}
						}
					}
				}
			}
		}
	}

	static FAutoConsoleVariableSink CVarSink(FConsoleCommandDelegate::CreateStatic(&CVarSinkFunction));

#endif
}

struct FFoliagePlacementUtil
{
	static int32 GetRandomSeedForPosition(const FVector2D& Position)
	{
		// generate a unique random seed for a given position (precision = cm)
		int64 Xcm = FMath::RoundToInt(Position.X);
		int64 Ycm = FMath::RoundToInt(Position.Y);
		// use the int32 hashing function to avoid patterns by spreading out distribution : 
		return HashCombine(GetTypeHash(Xcm), GetTypeHash(Ycm));
	}
};

// Legacy (< FFoliageCustomVersion::CrossLevelBase) serializer
FArchive& operator<<(FArchive& Ar, FFoliageInstance_Deprecated& Instance)
{
	Ar << Instance.Base;
	Ar << Instance.Location;
	Ar << Instance.Rotation;
	Ar << Instance.DrawScale3D;

	if (Ar.CustomVer(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::FoliageUsingHierarchicalISMC)
	{
		int32 OldClusterIndex;
		Ar << OldClusterIndex;
		Ar << Instance.PreAlignRotation;
		Ar << Instance.Flags;

		if (OldClusterIndex == INDEX_NONE)
		{
			// When converting, we need to skip over any instance that was previously deleted but still in the Instances array.
			Instance.Flags |= FOLIAGE_InstanceDeleted;
		}
	}
	else
	{
		Ar << Instance.PreAlignRotation;
		Ar << Instance.Flags;
	}

	Ar << Instance.ZOffset;

#if WITH_EDITORONLY_DATA
	if (!Ar.ArIsFilterEditorOnly && Ar.CustomVer(FFoliageCustomVersion::GUID) >= FFoliageCustomVersion::ProceduralGuid)
	{
		Ar << Instance.ProceduralGuid;
	}
#endif

	return Ar;
}

//
// Serializers for struct data
//
FArchive& operator<<(FArchive& Ar, FFoliageInstance& Instance)
{
	Ar << Instance.Location;
	Ar << Instance.Rotation;
	Ar << Instance.DrawScale3D;
	Ar << Instance.PreAlignRotation;
	Ar << Instance.ProceduralGuid;
	Ar << Instance.Flags;
	Ar << Instance.ZOffset;
	Ar << Instance.BaseId;

	return Ar;
}

static void ConvertDeprecatedFoliageMeshes(
	AInstancedFoliageActor* IFA,
	const TMap<UFoliageType*, TUniqueObj<FFoliageMeshInfo_Deprecated>>& FoliageMeshesDeprecated,
	const TMap<UFoliageType*, TUniqueObj<FFoliageInfo>>& FoliageInfos)
{
#if WITH_EDITORONLY_DATA	
	for (auto Pair : FoliageMeshesDeprecated)
	{
		auto& FoliageMesh = IFA->AddFoliageInfo(Pair.Key);
		const auto& FoliageMeshDeprecated = Pair.Value;

		// Old Foliage mesh is always static mesh (no actors)
		FoliageMesh->Type = EFoliageImplType::StaticMesh;
		FoliageMesh->Implementation.Reset(new FFoliageStaticMesh(&FoliageMesh.Get(), FoliageMeshDeprecated->Component));
		FoliageMesh->FoliageTypeUpdateGuid = FoliageMeshDeprecated->FoliageTypeUpdateGuid;

		FoliageMesh->Instances.Reserve(FoliageMeshDeprecated->Instances.Num());

		for (const FFoliageInstance_Deprecated& DeprecatedInstance : FoliageMeshDeprecated->Instances)
		{
			FFoliageInstance Instance;
			static_cast<FFoliageInstancePlacementInfo&>(Instance) = DeprecatedInstance;
			Instance.BaseId = IFA->InstanceBaseCache.AddInstanceBaseId(DeprecatedInstance.Base);
			Instance.ProceduralGuid = DeprecatedInstance.ProceduralGuid;

			FoliageMesh->Instances.Add(Instance);
		}
	}

	// there were no cross-level references before
	check(IFA->InstanceBaseCache.InstanceBaseLevelMap.Num() <= 1);
	// populate WorldAsset->BasePtr map
	IFA->InstanceBaseCache.InstanceBaseLevelMap.Empty();
	auto& BaseList = IFA->InstanceBaseCache.InstanceBaseLevelMap.Add(TSoftObjectPtr<UWorld>(Cast<UWorld>(IFA->GetLevel()->GetOuter())));
	for (auto& BaseInfoPair : IFA->InstanceBaseCache.InstanceBaseMap)
	{
		BaseList.Add(BaseInfoPair.Value.BasePtr);
	}
#endif//WITH_EDITORONLY_DATA	
}

static void ConvertDeprecated2FoliageMeshes(
	AInstancedFoliageActor* IFA,
	const TMap<UFoliageType*, TUniqueObj<FFoliageMeshInfo_Deprecated2>>& FoliageMeshesDeprecated,
	const TMap<UFoliageType*, TUniqueObj<FFoliageInfo>>& FoliageInfos)
{
#if WITH_EDITORONLY_DATA	
	for (auto Pair : FoliageMeshesDeprecated)
	{
		UFoliageType* FoliageType = Pair.Key;
		TUniqueObj<FFoliageMeshInfo_Deprecated2>& FoliageMeshDeprecated = Pair.Value;
		
		if (!FoliageType)
		{
			FMessageLog("MapCheck").Warning()
				->AddToken(FUObjectToken::Create(IFA))
				->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_FoliageMissingFoliageType", "Foliage instances for a missing foliage type have been removed.")))
				->AddToken(FMapErrorToken::Create(FMapErrors::FoliageMissingStaticMesh));

			if (FoliageMeshDeprecated->Component)
			{
				FoliageMeshDeprecated->Component->SetFlags(RF_Transactional);
				FoliageMeshDeprecated->Component->Modify();
				FoliageMeshDeprecated->Component->DestroyComponent();
				FoliageMeshDeprecated->Component = nullptr;
			}

			continue;
		}

		auto& FoliageMesh = IFA->AddFoliageInfo(Pair.Key);

		// Old Foliage mesh is always static mesh (no actors)
		FoliageMesh->Type = EFoliageImplType::StaticMesh;
		FoliageMesh->Implementation.Reset(new FFoliageStaticMesh(&FoliageMesh.Get(), FoliageMeshDeprecated->Component));
		FoliageMesh->FoliageTypeUpdateGuid = FoliageMeshDeprecated->FoliageTypeUpdateGuid;

		FoliageMesh->Instances.Reserve(FoliageMeshDeprecated->Instances.Num());

		for (const FFoliageInstance& Instance : FoliageMeshDeprecated->Instances)
		{
			FoliageMesh->Instances.Add(Instance);
		}
	}
#endif//WITH_EDITORONLY_DATA	
}

/**
*	FFoliageInstanceCluster_Deprecated
*/
struct FFoliageInstanceCluster_Deprecated
{
	UInstancedStaticMeshComponent* ClusterComponent;
	FBoxSphereBounds Bounds;

#if WITH_EDITORONLY_DATA
	TArray<int32> InstanceIndices;	// index into editor editor Instances array
#endif

	friend FArchive& operator<<(FArchive& Ar, FFoliageInstanceCluster_Deprecated& OldCluster)
	{
		check(Ar.CustomVer(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::FoliageUsingHierarchicalISMC);

		Ar << OldCluster.Bounds;
		Ar << OldCluster.ClusterComponent;

#if WITH_EDITORONLY_DATA
		if (!Ar.ArIsFilterEditorOnly ||
			Ar.UEVer() < VER_UE4_FOLIAGE_SETTINGS_TYPE)
		{
			Ar << OldCluster.InstanceIndices;
		}
#endif

		return Ar;
	}
};

FArchive& operator<<(FArchive& Ar, FFoliageMeshInfo_Deprecated& MeshInfo)
{
	if (Ar.CustomVer(FFoliageCustomVersion::GUID) >= FFoliageCustomVersion::FoliageUsingHierarchicalISMC)
	{
		Ar << MeshInfo.Component;
	}
	else
	{
		TArray<FFoliageInstanceCluster_Deprecated> OldInstanceClusters;
		Ar << OldInstanceClusters;
	}

#if WITH_EDITORONLY_DATA
	if ((!Ar.ArIsFilterEditorOnly || Ar.UEVer() < VER_UE4_FOLIAGE_SETTINGS_TYPE) &&
		(!(Ar.GetPortFlags() & PPF_DuplicateForPIE)))
	{
		Ar << MeshInfo.Instances;
	}

	if (!Ar.ArIsFilterEditorOnly && Ar.CustomVer(FFoliageCustomVersion::GUID) >= FFoliageCustomVersion::AddedFoliageTypeUpdateGuid)
	{
		Ar << MeshInfo.FoliageTypeUpdateGuid;
	}
#endif

	return Ar;
}

FFoliageMeshInfo_Deprecated2::FFoliageMeshInfo_Deprecated2()
	: Component(nullptr)
{ }

FArchive& operator<<(FArchive& Ar, FFoliageMeshInfo_Deprecated2& MeshInfo)
{
	Ar << MeshInfo.Component;

#if WITH_EDITORONLY_DATA
	Ar << MeshInfo.Instances;
	Ar << MeshInfo.FoliageTypeUpdateGuid;
#endif

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FFoliageInfo& Info)
{
	Ar << Info.Type;
	if (Ar.IsLoading() || (Ar.IsTransacting() && !Info.Implementation.IsValid()))
	{
		Info.CreateImplementation(Info.Type);
	}
	
	if (Info.Implementation)
	{
		Info.Implementation->Serialize(Ar);
	}

#if WITH_EDITORONLY_DATA
	if (!Ar.ArIsFilterEditorOnly && !(Ar.GetPortFlags() & PPF_DuplicateForPIE))
	{
		if (Ar.IsTransacting())
		{
			Info.Instances.BulkSerialize(Ar, Ar.UEVer() < EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES);
		}
		else
		{
			Ar << Info.Instances;
		}
	}

	if (!Ar.ArIsFilterEditorOnly)
	{
		Ar << Info.FoliageTypeUpdateGuid;
	}

	// Serialize the transient data for undo.
	if (Ar.IsTransacting())
	{
		Ar << Info.ComponentHash;
		Ar << Info.SelectedIndices;
	}
#endif

	return Ar;
}

//
// FFoliageDensityFalloff
//

FFoliageDensityFalloff::FFoliageDensityFalloff()
{
	FRichCurve* FalloffRichCurve = FalloffCurve.GetRichCurve();
	FalloffRichCurve->AddKey(0.f, 1.f);
	FalloffRichCurve->AddKey(1.f, 0.f);
}

bool FFoliageDensityFalloff::IsInstanceFiltered(const FVector2D& InstancePosition, const FVector2D& Origin, FVector::FReal MaxDistance) const
{
	float KeepPointProbability = GetDensityFalloffValue(InstancePosition, Origin, MaxDistance);
	check(KeepPointProbability >= 0.f && KeepPointProbability <= 1.f);
	if (KeepPointProbability < 1.f)
	{
		int32 PointSeed = FFoliagePlacementUtil::GetRandomSeedForPosition(InstancePosition);
		FRandomStream LocalRandomStream(PointSeed);
		float Rand = LocalRandomStream.FRand();
		return (Rand > KeepPointProbability);
	}
	return false;
}

float FFoliageDensityFalloff::GetDensityFalloffValue(const FVector2D& Position, const FVector2D& Origin, FVector::FReal MaxDistance) const
{
	float KeepPointProbability = 1.f;
	if (bUseFalloffCurve)
	{
		FVector::FReal Distance = FVector2D::Distance(Position, Origin);
		float NormalizedDistance = static_cast<float>(MaxDistance > 0.f ? (Distance / MaxDistance) : 1.f);
		if (NormalizedDistance > 1.f)
			NormalizedDistance = 1.f;
		const FRichCurve* FalloffRichCurve = FalloffCurve.GetRichCurveConst();
		KeepPointProbability = FMath::Clamp(FalloffRichCurve->Eval(NormalizedDistance), 0.f, 1.f);
	}
	return KeepPointProbability;
}

//
// UFoliageType
//

UFoliageType::UFoliageType(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Density = 100.0f;
	Radius = 0.0f;
	AlignToNormal = true;
	RandomYaw = true;
	Scaling = EFoliageScaling::Uniform;
	ScaleX.Min = 1.0f;
	ScaleY.Min = 1.0f;
	ScaleZ.Min = 1.0f;
	ScaleX.Max = 1.0f;
	ScaleY.Max = 1.0f;
	ScaleZ.Max = 1.0f;
	AlignMaxAngle = 0.0f;
	RandomPitchAngle = 0.0f;
	GroundSlopeAngle.Min = 0.0f;
	GroundSlopeAngle.Max = 45.0f;
	Height.Min = -262144.0f;
	Height.Max = 262144.0f;
	ZOffset.Min = 0.0f;
	ZOffset.Max = 0.0f;
	CullDistance.Min = 0;
	CullDistance.Max = 0;
	bEnableStaticLighting_DEPRECATED = true;
	MinimumLayerWeight = 0.5f;
	AverageNormal = false;
	AverageNormalSingleComponent = true;
	AverageNormalSampleCount = 10;
#if WITH_EDITORONLY_DATA
	IsSelected = false;
#endif
	DensityAdjustmentFactor = 1.0f;
	CollisionWithWorld = false;
	CollisionScale = FVector(0.9f, 0.9f, 0.9f);

	Mobility = EComponentMobility::Static;
	CastShadow = true;
	bCastDynamicShadow = true;
	bCastStaticShadow = true;
	bCastContactShadow = true;
	bAffectDynamicIndirectLighting = true;
	// Most of the high instance count foliage like grass causes performance problems with distance field lighting
	bAffectDistanceFieldLighting = false;
	bCastShadowAsTwoSided = false;
	bReceivesDecals = false;

	TranslucencySortPriority = 0;

	bOverrideLightMapRes = false;
	OverriddenLightMapRes = 8;
	bUseAsOccluder = false;
	bVisibleInRayTracing = true;
	bVisibleInReflections = true;
	bEvaluateWorldPositionOffset = true;
	WorldPositionOffsetDisableDistance = 0;
	NanitePixelProgrammableDistance = 0.f;

	ShadowCacheInvalidationBehavior = EShadowCacheInvalidationBehavior::Auto;

	BodyInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	/** Ecosystem settings*/
	AverageSpreadDistance = 50;
	SpreadVariance = 150;
	bCanGrowInShade = false;
	bSpawnsInShade = false;
	SeedsPerStep = 3;
	OverlapPriority = 0.f;
	NumSteps = 3;
	ProceduralScale = FFloatInterval(1.f, 3.f);
	ChangeCount = 0;
	InitialSeedDensity = 1.f;
	CollisionRadius = 100.f;
	ShadeRadius = 100.f;
	MaxInitialAge = 0.f;
	MaxAge = 10.f;

	FRichCurve* Curve = ScaleCurve.GetRichCurve();
	Curve->AddKey(0.f, 0.f);
	Curve->AddKey(1.f, 1.f);

	if(!IsTemplate())
	{
		UpdateGuid = FGuid::NewGuid();
	}
#if WITH_EDITORONLY_DATA
	HiddenEditorViews = 0;
#endif
	bEnableDensityScaling = false;
	bEnableDiscardOnLoad = false;
	bEnableCullDistanceScaling = false;

#if WITH_EDITORONLY_DATA
	bIncludeInHLOD = true;
#endif

#if WITH_EDITORONLY_DATA
	// Deprecated since FFoliageCustomVersion::FoliageTypeCustomization
	ScaleMinX_DEPRECATED = 1.0f;
	ScaleMinY_DEPRECATED = 1.0f;
	ScaleMinZ_DEPRECATED = 1.0f;
	ScaleMaxX_DEPRECATED = 1.0f;
	ScaleMaxY_DEPRECATED = 1.0f;
	ScaleMaxZ_DEPRECATED = 1.0f;
	HeightMin_DEPRECATED = -262144.0f;
	HeightMax_DEPRECATED = 262144.0f;
	ZOffsetMin_DEPRECATED = 0.0f;
	ZOffsetMax_DEPRECATED = 0.0f;
	UniformScale_DEPRECATED = true;
	GroundSlope_DEPRECATED = 45.0f;

	// Deprecated since FFoliageCustomVersion::FoliageTypeProceduralScaleAndShade
	MinScale_DEPRECATED = 1.f;
	MaxScale_DEPRECATED = 3.f;

#endif// WITH_EDITORONLY_DATA
}

void UFoliageType::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFoliageCustomVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	// we now have mask configurations for every color channel
	if (Ar.IsLoading() && Ar.IsPersistent() && !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE) && VertexColorMask_DEPRECATED != FOLIAGEVERTEXCOLORMASK_Disabled)
	{
		FFoliageVertexColorChannelMask* Mask = nullptr;
		switch (VertexColorMask_DEPRECATED)
		{
		case FOLIAGEVERTEXCOLORMASK_Red:
			Mask = &VertexColorMaskByChannel[(uint8)EVertexColorMaskChannel::Red];
			break;

		case FOLIAGEVERTEXCOLORMASK_Green:
			Mask = &VertexColorMaskByChannel[(uint8)EVertexColorMaskChannel::Green];
			break;

		case FOLIAGEVERTEXCOLORMASK_Blue:
			Mask = &VertexColorMaskByChannel[(uint8)EVertexColorMaskChannel::Blue];
			break;

		case FOLIAGEVERTEXCOLORMASK_Alpha:
			Mask = &VertexColorMaskByChannel[(uint8)EVertexColorMaskChannel::Alpha];
			break;
		}

		if (Mask != nullptr)
		{
			Mask->UseMask = true;
			Mask->MaskThreshold = VertexColorMaskThreshold_DEPRECATED;
			Mask->InvertMask = VertexColorMaskInvert_DEPRECATED;

			VertexColorMask_DEPRECATED = FOLIAGEVERTEXCOLORMASK_Disabled;
		}
	}

	if (LandscapeLayer_DEPRECATED != NAME_None && LandscapeLayers.Num() == 0)	//we now store an array of names so initialize the array with the old name
	{
		LandscapeLayers.Add(LandscapeLayer_DEPRECATED);
		LandscapeLayer_DEPRECATED = NAME_None;
	}

	if (Ar.IsLoading() && GetLinkerCustomVersion(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::AddedMobility)
	{
		Mobility = bEnableStaticLighting_DEPRECATED ? EComponentMobility::Static : EComponentMobility::Movable;
	}

	// Discard scalable Foliage data on load
	if (Ar.IsLoading() && GetLinkerCustomVersion(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::FoliageDiscardOnLoad)
	{
		bEnableDiscardOnLoad = bEnableDensityScaling;
	}

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::FoliageTypeCustomization)
		{
			ScaleX.Min = ScaleMinX_DEPRECATED;
			ScaleX.Max = ScaleMaxX_DEPRECATED;

			ScaleY.Min = ScaleMinY_DEPRECATED;
			ScaleY.Max = ScaleMaxY_DEPRECATED;

			ScaleZ.Min = ScaleMinZ_DEPRECATED;
			ScaleZ.Max = ScaleMaxZ_DEPRECATED;

			Height.Min = HeightMin_DEPRECATED;
			Height.Max = HeightMax_DEPRECATED;

			ZOffset.Min = ZOffsetMin_DEPRECATED;
			ZOffset.Max = ZOffsetMax_DEPRECATED;

			CullDistance.Min = StartCullDistance_DEPRECATED;
			CullDistance.Max = EndCullDistance_DEPRECATED;
		}

		if (Ar.CustomVer(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::FoliageTypeCustomizationScaling)
		{
			Scaling = UniformScale_DEPRECATED ? EFoliageScaling::Uniform : EFoliageScaling::Free;

			GroundSlopeAngle.Min = MinGroundSlope_DEPRECATED;
			GroundSlopeAngle.Max = GroundSlope_DEPRECATED;
		}

		if (Ar.CustomVer(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::FoliageTypeProceduralScaleAndShade)
		{
			bCanGrowInShade = bSpawnsInShade;

			ProceduralScale.Min = MinScale_DEPRECATED;
			ProceduralScale.Max = MaxScale_DEPRECATED;
		}

		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::FoliageTypeIncludeInHLOD)
		{
			bIncludeInHLOD = false;
		}
	}
#endif// WITH_EDITORONLY_DATA
}

void UFoliageType::PostLoad()
{
	Super::PostLoad();

	if (!IsTemplate())
	{
		BodyInstance.FixupData(this);

	}
}

bool UFoliageType::IsNotAssetOrBlueprint() const
{
	return IsAsset() == false && GetClass()->IsNative();
}


FVector3f UFoliageType::GetRandomScale() const
{
	FVector3f Result(1.0f);
	float LockRand = 0.0f;

	switch (Scaling)
	{
	case EFoliageScaling::Uniform:
		Result.X = ScaleX.Interpolate(FMath::FRand());
		Result.Y = Result.X;
		Result.Z = Result.X;
		break;

	case EFoliageScaling::Free:
		Result.X = ScaleX.Interpolate(FMath::FRand());
		Result.Y = ScaleY.Interpolate(FMath::FRand());
		Result.Z = ScaleZ.Interpolate(FMath::FRand());
		break;

	case EFoliageScaling::LockXY:
		LockRand = FMath::FRand();
		Result.X = ScaleX.Interpolate(LockRand);
		Result.Y = ScaleY.Interpolate(LockRand);
		Result.Z = ScaleZ.Interpolate(FMath::FRand());
		break;

	case EFoliageScaling::LockXZ:
		LockRand = FMath::FRand();
		Result.X = ScaleX.Interpolate(LockRand);
		Result.Y = ScaleY.Interpolate(FMath::FRand());
		Result.Z = ScaleZ.Interpolate(LockRand);

	case EFoliageScaling::LockYZ:
		LockRand = FMath::FRand();
		Result.X = ScaleX.Interpolate(FMath::FRand());
		Result.Y = ScaleY.Interpolate(LockRand);
		Result.Z = ScaleZ.Interpolate(LockRand);
	}

	return Result;
}

UFoliageType_InstancedStaticMesh::UFoliageType_InstancedStaticMesh(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Mesh = nullptr;
	ComponentClass = UFoliageInstancedStaticMeshComponent::StaticClass();
	CustomNavigableGeometry = EHasCustomNavigableGeometry::Yes;
}

UObject* UFoliageType_InstancedStaticMesh::GetSource() const
{
	return Cast<UObject>(GetStaticMesh());
}

#if WITH_EDITOR
void UFoliageType_InstancedStaticMesh::SetSource(UObject* InSource)
{
	UStaticMesh* InMesh = Cast<UStaticMesh>(InSource);
	check(InSource == nullptr || InMesh != nullptr);
	SetStaticMesh(InMesh);
}

void UFoliageType_InstancedStaticMesh::UpdateBounds()
{
	if (Mesh == nullptr)
	{
		return;
	}

	MeshBounds = Mesh->GetBounds();

	// Make bottom only bound
	FBox LowBound = MeshBounds.GetBox();
	LowBound.Max.Z = LowBound.Min.Z + (LowBound.Max.Z - LowBound.Min.Z) * 0.1f;

	float MinX = FLT_MAX, MaxX = FLT_MIN, MinY = FLT_MAX, MaxY = FLT_MIN;
	LowBoundOriginRadius = FVector::ZeroVector;

	if (Mesh->GetRenderData())
	{
		FPositionVertexBuffer& PositionVertexBuffer = Mesh->GetRenderData()->LODResources[0].VertexBuffers.PositionVertexBuffer;
		for (uint32 Index = 0; Index < PositionVertexBuffer.GetNumVertices(); ++Index)
		{
			const FVector3f& Pos = PositionVertexBuffer.VertexPosition(Index);
			if (Pos.Z < LowBound.Max.Z)
			{
				MinX = FMath::Min(MinX, Pos.X);
				MinY = FMath::Min(MinY, Pos.Y);
				MaxX = FMath::Max(MaxX, Pos.X);
				MaxY = FMath::Max(MaxY, Pos.Y);
			}
		}
	}

	LowBoundOriginRadius = FVector((MinX + MaxX), (MinY + MaxY), FMath::Sqrt(FMath::Square(MaxX - MinX) + FMath::Square(MaxY - MinY))) * 0.5f;
}
#endif

UFoliageType_Actor::UFoliageType_Actor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Density = 10;
	Radius = 500;
	bShouldAttachToBaseComponent = true;
	StaticMeshOnlyComponentClass = UFoliageInstancedStaticMeshComponent::StaticClass();
}

#if WITH_EDITOR
void UFoliageType_Actor::UpdateBounds()
{
	if (ActorClass == nullptr)
	{
		return;
	}

	FPreviewScene PreviewScene;
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	AActor* PreviewActor = PreviewScene.GetWorld()->SpawnActor<AActor>(ActorClass, SpawnInfo);
	if (PreviewActor == nullptr)
	{
		return;
	}
	
	PreviewActor->SetActorEnableCollision(false);

	// Put this in method...
	FBoxSphereBounds::Builder MeshBoundsBuilder;
	if (PreviewActor != nullptr && PreviewActor->GetRootComponent())
	{
		TArray<USceneComponent*> PreviewComponents;
		PreviewActor->GetRootComponent()->GetChildrenComponents(true, PreviewComponents);
		PreviewComponents.Add(PreviewActor->GetRootComponent());

		for (USceneComponent* PreviewComponent : PreviewComponents)
		{
			if (!(PreviewComponent->bIsEditorOnly || PreviewComponent->bHiddenInGame))
			{
				MeshBoundsBuilder += PreviewComponent->Bounds;
			}
		}
	}
	MeshBounds = MeshBoundsBuilder;

	FBox LowBound = MeshBounds.GetBox();
	LowBound.Max.Z = LowBound.Min.Z + (LowBound.Max.Z - LowBound.Min.Z) * 0.1f;

	FVector::FReal MinX = LowBound.Min.X, MaxX = LowBound.Max.X, MinY = LowBound.Min.Y, MaxY = LowBound.Max.Y;
	LowBoundOriginRadius = FVector::ZeroVector;

	// TODO: Get more precise lower bound from multiple possible meshes in Actor

	LowBoundOriginRadius = FVector((MinX + MaxX), (MinY + MaxY), FMath::Sqrt(FMath::Square(MaxX - MinX) + FMath::Square(MaxY - MinY))) * 0.5f;

	PreviewActor->Destroy();
}
#endif

float UFoliageType::GetMaxRadius() const
{
	return FMath::Max(CollisionRadius, ShadeRadius);
}

float UFoliageType::GetScaleForAge(const float Age) const
{
	const FRichCurve* Curve = ScaleCurve.GetRichCurveConst();
	const float Time = FMath::Clamp(MaxAge == 0 ? 1.f : Age / MaxAge, 0.f, 1.f);
	const float Scale = Curve->Eval(Time);
	return ProceduralScale.Min + ProceduralScale.Size() * Scale;
}

float UFoliageType::GetInitAge(FRandomStream& RandomStream) const
{
	return MaxInitialAge * RandomStream.GetFraction();
}

float UFoliageType::GetNextAge(const float CurrentAge, const int32 InNumSteps) const
{
	float NewAge = CurrentAge;
	for (int32 Count = 0; Count < InNumSteps; ++Count)
	{
		const float GrowAge = NewAge + 1;
		if (GrowAge <= MaxAge)
		{
			NewAge = GrowAge;
		}
		else
		{
			break;
		}
	}

	return NewAge;
}

bool UFoliageType::GetSpawnsInShade() const
{
	return bCanGrowInShade && bSpawnsInShade;
}

#if WITH_EDITOR
void UFoliageType::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Ensure that OverriddenLightMapRes is a factor of 4
	OverriddenLightMapRes = OverriddenLightMapRes > 4 ? OverriddenLightMapRes + 3 & ~3 : 4;
	++ChangeCount;

	UpdateGuid = FGuid::NewGuid();

	const bool bSourceChanged = IsSourcePropertyChange(PropertyChangedEvent.Property);
	if (bSourceChanged)
	{
		UpdateBounds();
	}

	// Notify any currently-loaded InstancedFoliageActors
	if (IsFoliageReallocationRequiredForPropertyChange(PropertyChangedEvent.Property))
	{
		for (TObjectIterator<AInstancedFoliageActor> It(RF_ClassDefaultObject, /** bIncludeDerivedClasses */ true, /** InternalExcludeFalgs */ EInternalObjectFlags::Garbage); It; ++It)
		{
			if (It->GetWorld() != nullptr)
			{
				It->NotifyFoliageTypeChanged(this, bSourceChanged);
			}
		}
	}
}

void UFoliageType::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (IsSourcePropertyChange(PropertyAboutToChange))
	{
		for (TObjectIterator<AInstancedFoliageActor> It(RF_ClassDefaultObject, /** bIncludeDerivedClasses */ true, /** InternalExcludeFalgs */ EInternalObjectFlags::Garbage); It; ++It)
		{
			It->NotifyFoliageTypeWillChange(this);
		}
	}
}

void UFoliageType::OnHiddenEditorViewMaskChanged(UWorld* InWorld)
{
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		FFoliageInfo* Info = It->FindInfo(this);
		if (Info != nullptr)
		{
			Info->OnHiddenEditorViewMaskChanged(HiddenEditorViews);
		}
	}
}

FName UFoliageType::GetDisplayFName() const
{
	FName DisplayFName;

	if (IsAsset())
	{
		DisplayFName = GetFName();
	}
	else if (UBlueprint* FoliageTypeBP = Cast<UBlueprint>(GetClass()->ClassGeneratedBy))
	{
		DisplayFName = FoliageTypeBP->GetFName();
	}
	else if (UObject* Source = GetSource())
	{
		DisplayFName = Source->GetFName();
	}

	return DisplayFName;
}

#endif

//
// FFoliageStaticMesh
//
FFoliageStaticMesh::~FFoliageStaticMesh()
{
#if WITH_EDITOR
	if (Component != nullptr)
	{
		if (Component->GetStaticMesh() != nullptr)
		{
			Component->GetStaticMesh()->GetOnExtendedBoundsChanged().RemoveAll(this);
		}
	}
#endif
}

void FFoliageStaticMesh::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	if (Component != nullptr)
	{
		Collector.AddReferencedObject(Component, InThis);
	}
}

void FFoliageStaticMesh::Serialize(FArchive& Ar)
{
	Ar << Component;
}

int32 FFoliageStaticMesh::GetOverlappingSphereCount(const FSphere& Sphere) const
{
	if (Component && Component->IsTreeFullyBuilt())
	{
		return Component->GetOverlappingSphereCount(Sphere);
	}
	return 0;
}

int32 FFoliageStaticMesh::GetOverlappingBoxCount(const FBox& Box) const
{
	if (Component && Component->IsTreeFullyBuilt())
	{
		return Component->GetOverlappingBoxCount(Box);
	}
	return 0;
}
void FFoliageStaticMesh::GetOverlappingBoxTransforms(const FBox& Box, TArray<FTransform>& OutTransforms) const
{
	if (Component && Component->IsTreeFullyBuilt())
	{
		Component->GetOverlappingBoxTransforms(Box, OutTransforms);
	}
}
void FFoliageStaticMesh::GetOverlappingMeshCount(const FSphere& Sphere, TMap<UStaticMesh*, int32>& OutCounts) const
{
	int32 Count = GetOverlappingSphereCount(Sphere);
	if (Count > 0)
	{
		UStaticMesh* const Mesh = Component->GetStaticMesh();
		int32& StoredCount = OutCounts.FindOrAdd(Mesh);
		StoredCount += Count;
	}
}

#if WITH_EDITOR
void FFoliageStaticMesh::Initialize(const UFoliageType* FoliageType)
{
	CreateNewComponent(FoliageType);
}

void FFoliageStaticMesh::Uninitialize()
{
	if (Component != nullptr)
	{
		if (Component->GetStaticMesh() != nullptr)
		{
			Component->GetStaticMesh()->GetOnExtendedBoundsChanged().RemoveAll(this);
		}

		Component->ClearInstances();
		Component->SetFlags(RF_Transactional);
		Component->Modify();
		Component->DestroyComponent();
		Component = nullptr;
	}
}

int32 FFoliageStaticMesh::GetInstanceCount() const
{
	if (Component != nullptr)
	{
		return Component->GetInstanceCount();
	}

	return 0;
}

void FFoliageStaticMesh::PreAddInstances(const UFoliageType* FoliageType, int32 Count)
{
	if (!IsInitialized())
	{
		Initialize(FoliageType);
		check(IsInitialized());
	}
	else
	{
		Component->InvalidateLightingCache();
	}

	if (Count)
	{
		Component->PreAllocateInstancesMemory(Count);
	}
}

void FFoliageStaticMesh::AddInstance(const FFoliageInstance& NewInstance)
{
	check(Component);
	Component->AddInstance(NewInstance.GetInstanceWorldTransform(), /*bWorldSpace*/true);
	bInvalidateLightingCache = true;
}

void FFoliageStaticMesh::RemoveInstance(int32 InstanceIndex)
{
	check(Component);
	Component->RemoveInstance(InstanceIndex);

	if (UpdateDepth > 0)
	{
		bInvalidateLightingCache = true;
	}
	else
	{
		Component->InvalidateLightingCache();
	}
}

void FFoliageStaticMesh::SetInstanceWorldTransform(int32 InstanceIndex, const FTransform& Transform, bool bTeleport)
{
	check(Component);
	Component->UpdateInstanceTransform(InstanceIndex, Transform, true, true, bTeleport);
	bInvalidateLightingCache = true;
}

FTransform FFoliageStaticMesh::GetInstanceWorldTransform(int32 InstanceIndex) const
{
	return FTransform(Component->PerInstanceSMData[InstanceIndex].Transform) * Component->GetComponentToWorld();
}

void FFoliageStaticMesh::PostUpdateInstances()
{
	check(Component);
	Component->InvalidateLightingCache();
	Component->MarkRenderStateDirty();
}

bool FFoliageStaticMesh::IsOwnedComponent(const UPrimitiveComponent* PrimitiveComponent) const
{
	return Component == PrimitiveComponent;
}

void FFoliageStaticMesh::SelectAllInstances(bool bSelect)
{
	check(Component);
	Component->SelectInstance(bSelect, 0, Component->GetInstanceCount());
	Component->MarkRenderStateDirty();
}

void FFoliageStaticMesh::SelectInstance(bool bSelect, int32 Index)
{
	check(Component);
	Component->SelectInstance(bSelect, Index);
	Component->MarkRenderStateDirty();
}

void FFoliageStaticMesh::SelectInstances(bool bSelect, const TSet<int32>& SelectedIndices)
{
	check(Component);
	for (int32 i : SelectedIndices)
	{
		Component->SelectInstance(bSelect, i);
	}
	Component->MarkRenderStateDirty();
}

int32 FFoliageStaticMesh::GetInstanceIndexFrom(const UPrimitiveComponent* PrimitiveComponent, int32 ComponentIndex) const
{
	if (IsOwnedComponent(PrimitiveComponent))
	{
		return ComponentIndex;
	}

	return INDEX_NONE;
}

FBox FFoliageStaticMesh::GetSelectionBoundingBox(const TSet<int32>& SelectedIndices) const
{
	FBox BoundingBox(EForceInit::ForceInit);
	for (int32 i : SelectedIndices)
	{
		FTransform InstanceWorldTransform;
		Component->GetInstanceTransform(i, InstanceWorldTransform, true);
		BoundingBox += Component->GetStaticMesh()->GetBoundingBox().TransformBy(InstanceWorldTransform);
	}
	return BoundingBox;
}

void FFoliageStaticMesh::ApplySelection(bool bApply, const TSet<int32>& SelectedIndices)
{
	if (Component && (bApply || Component->SelectedInstances.Num() > 0))
	{
		Component->ClearInstanceSelection();

		if (bApply)
		{
			for (int32 i : SelectedIndices)
			{
				Component->SelectInstance(true, i, 1);
			}
		}

		Component->MarkRenderStateDirty();
	}
}

void FFoliageStaticMesh::ClearSelection(const TSet<int32>& SelectedIndices)
{
	check(Component);
	Component->ClearInstanceSelection();
	Component->MarkRenderStateDirty();
}

void FFoliageStaticMesh::ForEachSMInstance(TFunctionRef<bool(FSMInstanceId)> Callback) const
{
	if (Component)
	{
		for (int32 InstanceIndex = 0; Info->Instances.Num(); ++InstanceIndex)
		{
			if (!Callback(FSMInstanceId{ Component, InstanceIndex }))
			{
				return;
			}
		}
	}
}

void FFoliageStaticMesh::ForEachSMInstance(int32 InstanceIndex, TFunctionRef<bool(FSMInstanceId)> Callback) const
{
	if (Component && Info->Instances.IsValidIndex(InstanceIndex))
	{
		Callback(FSMInstanceId{ Component, InstanceIndex });
	}
}

void FFoliageStaticMesh::BeginUpdate()
{
	if (UpdateDepth == 0)
	{
		if (Component)
		{
			bPreviousValue = Component->bAutoRebuildTreeOnInstanceChanges;
			Component->bAutoRebuildTreeOnInstanceChanges = false;
		}
		else
		{
			// The default value for HISM component is true, and if we add a component in between the BeginUpdate/EndUpdate pair, it makes sense also.
			bPreviousValue = true;
		}
	}
	++UpdateDepth;
}

void FFoliageStaticMesh::EndUpdate()
{
	check(UpdateDepth > 0);
	--UpdateDepth;

	if (UpdateDepth == 0 && Component)
	{
		Component->bAutoRebuildTreeOnInstanceChanges = bPreviousValue;

		if (bInvalidateLightingCache)
		{
			Component->InvalidateLightingCache();
			bInvalidateLightingCache = false;
		}
	}
}

void FFoliageStaticMesh::Refresh(bool Async, bool Force)
{
	if (Component != nullptr)
	{
		Component->BuildTreeIfOutdated(Async, Force);
	}
}

void FFoliageStaticMesh::OnHiddenEditorViewMaskChanged(uint64 InHiddenEditorViews)
{
	UFoliageInstancedStaticMeshComponent* FoliageComponent = Cast<UFoliageInstancedStaticMeshComponent>(Component);

	if (FoliageComponent && FoliageComponent->FoliageHiddenEditorViews != InHiddenEditorViews)
	{
		FoliageComponent->FoliageHiddenEditorViews = InHiddenEditorViews;
		FoliageComponent->MarkRenderStateDirty();
	}
}

void FFoliageStaticMesh::PreEditUndo(UFoliageType* FoliageType)
{
	if (UFoliageType_InstancedStaticMesh* FoliageType_InstancedStaticMesh = Cast<UFoliageType_InstancedStaticMesh>(FoliageType))
	{
		if (FoliageType_InstancedStaticMesh->GetStaticMesh() != nullptr)
		{
			FoliageType_InstancedStaticMesh->GetStaticMesh()->GetOnExtendedBoundsChanged().RemoveAll(this);
		}
	}
}

void FFoliageStaticMesh::PostEditUndo(FFoliageInfo* InInfo, UFoliageType* FoliageType)
{
	FFoliageImpl::PostEditUndo(InInfo, FoliageType);
	if (UFoliageType_InstancedStaticMesh* FoliageType_InstancedStaticMesh = Cast<UFoliageType_InstancedStaticMesh>(FoliageType))
	{
		if (Component != nullptr && FoliageType_InstancedStaticMesh->GetStaticMesh() != nullptr)
		{
			FoliageType_InstancedStaticMesh->GetStaticMesh()->GetOnExtendedBoundsChanged().AddRaw(this, &FFoliageStaticMesh::HandleComponentMeshBoundsChanged);
		}

		CheckComponentClass(FoliageType_InstancedStaticMesh);
		Reapply(FoliageType);
	}
}

void FFoliageStaticMesh::NotifyFoliageTypeWillChange(UFoliageType* FoliageType)
{
	if (Component != nullptr)
	{
		if (UFoliageType_InstancedStaticMesh* FoliageType_InstancedStaticMesh = Cast<UFoliageType_InstancedStaticMesh>(FoliageType))
		{
			if (FoliageType_InstancedStaticMesh->GetStaticMesh() != nullptr)
			{
				FoliageType_InstancedStaticMesh->GetStaticMesh()->GetOnExtendedBoundsChanged().RemoveAll(this);
			}
		}
	}
}

bool FFoliageStaticMesh::NotifyFoliageTypeChanged(UFoliageType* FoliageType, bool bSourceChanged)
{
	UFoliageType_InstancedStaticMesh* FoliageType_InstancedStaticMesh = Cast<UFoliageType_InstancedStaticMesh>(FoliageType);
	check(FoliageType_InstancedStaticMesh);
	CheckComponentClass(FoliageType_InstancedStaticMesh);
	UpdateComponentSettings(FoliageType_InstancedStaticMesh);
	
	if (bSourceChanged && Component != nullptr && Component->GetStaticMesh() != nullptr)
	{
		// Change bounds delegate bindings
		if (FoliageType_InstancedStaticMesh->GetStaticMesh() != nullptr)
		{
			Component->GetStaticMesh()->GetOnExtendedBoundsChanged().AddRaw(this, &FFoliageStaticMesh::HandleComponentMeshBoundsChanged);

			// Mesh changed, so we must update the occlusion tree
			Component->BuildTreeIfOutdated(true, false);
		}
	}

	return false;
}

void FFoliageStaticMesh::EnterEditMode()
{
	if (Component == nullptr)
	{
		return;
	}

	if (Component->GetStaticMesh() != nullptr)
	{
		Component->GetStaticMesh()->GetOnExtendedBoundsChanged().AddRaw(this, &FFoliageStaticMesh::HandleComponentMeshBoundsChanged);

		Component->BuildTreeIfOutdated(true, false);
	}

	Component->bCanEnableDensityScaling = false;
	Component->UpdateDensityScaling();
}

void FFoliageStaticMesh::ExitEditMode()
{
	if (Component == nullptr)
	{
		return;
	}

	if (Component->GetStaticMesh() != nullptr)
	{
		Component->GetStaticMesh()->GetOnExtendedBoundsChanged().RemoveAll(this);
	}

	Component->bCanEnableDensityScaling = true;
	Component->UpdateDensityScaling();
}

void FFoliageStaticMesh::CreateNewComponent(const UFoliageType* InSettings)
{
	SCOPE_CYCLE_COUNTER(STAT_FoliageCreateComponent);

	check(!Component);
	const UFoliageType_InstancedStaticMesh* FoliageType_InstancedStaticMesh = Cast<UFoliageType_InstancedStaticMesh>(InSettings);

	UClass* ComponentClass = FoliageType_InstancedStaticMesh->GetComponentClass();
	if (ComponentClass == nullptr)
	{
		ComponentClass = UFoliageInstancedStaticMeshComponent::StaticClass();
	}

	AInstancedFoliageActor* IFA = GetIFA();
	UFoliageInstancedStaticMeshComponent* FoliageComponent = NewObject<UFoliageInstancedStaticMeshComponent>(IFA, ComponentClass, NAME_None, RF_Transactional);
	IFA->AddInstanceComponent(FoliageComponent);
	
	check(FoliageType_InstancedStaticMesh);

	Component = FoliageComponent;
	Component->SetStaticMesh(FoliageType_InstancedStaticMesh->GetStaticMesh());
	Component->bSelectable = true;
	Component->bHasPerInstanceHitProxies = true;

	if (Component->GetStaticMesh() != nullptr)
	{
		Component->GetStaticMesh()->GetOnExtendedBoundsChanged().AddRaw(this, &FFoliageStaticMesh::HandleComponentMeshBoundsChanged);
	}

	FoliageComponent->FoliageHiddenEditorViews = InSettings->HiddenEditorViews;

	UpdateComponentSettings(FoliageType_InstancedStaticMesh);

	Component->SetupAttachment(IFA->GetRootComponent());

	if (IFA->GetRootComponent()->IsRegistered())
	{
		Component->RegisterComponent();
	}

	// Use only instance translation as a component transform
	Component->SetWorldTransform(IFA->GetRootComponent()->GetComponentTransform());

	// Add the new component to the transaction buffer so it will get destroyed on undo
	Component->Modify();
	// We don't want to track changes to instances later so we mark it as non-transactional
	Component->ClearFlags(RF_Transactional);
}

void FFoliageStaticMesh::HandleComponentMeshBoundsChanged(const FBoxSphereBounds& NewBounds)
{
	if (Component != nullptr)
	{
		Component->BuildTreeIfOutdated(true, false);
	}
}

void FFoliageStaticMesh::CheckComponentClass(const UFoliageType_InstancedStaticMesh* InSettings)
{
	if (Component)
	{
		UClass* ComponentClass = InSettings->GetComponentClass();
		if (ComponentClass == nullptr)
		{
			ComponentClass = UFoliageInstancedStaticMeshComponent::StaticClass();
		}

		if (ComponentClass != Component->GetClass())
		{
			AInstancedFoliageActor* IFA = GetIFA();
			IFA->Modify();

			// prepare to destroy the old component
			Uninitialize();

			// create a new component
			Initialize(InSettings);

			// apply the instances to it
			Reapply(InSettings);
		}
	}
}

void FFoliageStaticMesh::UpdateComponentSettings(const UFoliageType_InstancedStaticMesh* InSettings)
{
	if (Component)
	{
		bool bNeedsMarkRenderStateDirty = false;
		bool bNeedsInvalidateLightingCache = false;

		const UFoliageType_InstancedStaticMesh* FoliageType = InSettings;
		if (InSettings->GetClass()->ClassGeneratedBy)
		{
			// If we're updating settings for a BP foliage type, use the CDO
			FoliageType = InSettings->GetClass()->GetDefaultObject<UFoliageType_InstancedStaticMesh>();
		}

		if (Component->GetStaticMesh() != FoliageType->GetStaticMesh())
		{
			Component->SetStaticMesh(FoliageType->GetStaticMesh());

			bNeedsInvalidateLightingCache = true;
			bNeedsMarkRenderStateDirty = true;
		}

		if (Component->Mobility != FoliageType->Mobility)
		{
			Component->SetMobility(FoliageType->Mobility);
			bNeedsMarkRenderStateDirty = true;
			bNeedsInvalidateLightingCache = true;
		}

		const float FoliageCullDistanceScale = FMath::Clamp(CVarFoliageCullDistanceScale.GetValueOnGameThread(), 0.0f, 1.0f);
		FoliageUtil::UpdateComponentCullDistance(FoliageType, Component, FoliageCullDistanceScale);

		if (Component->CastShadow != FoliageType->CastShadow)
		{
			Component->CastShadow = FoliageType->CastShadow;
			bNeedsMarkRenderStateDirty = true;
			bNeedsInvalidateLightingCache = true;
		}
		if (Component->bCastDynamicShadow != FoliageType->bCastDynamicShadow)
		{
			Component->bCastDynamicShadow = FoliageType->bCastDynamicShadow;
			bNeedsMarkRenderStateDirty = true;
			bNeedsInvalidateLightingCache = true;
		}
		if (Component->bCastStaticShadow != FoliageType->bCastStaticShadow)
		{
			Component->bCastStaticShadow = FoliageType->bCastStaticShadow;
			bNeedsMarkRenderStateDirty = true;
			bNeedsInvalidateLightingCache = true;
		}
		if (Component->bCastContactShadow != FoliageType->bCastContactShadow)
		{
			Component->bCastContactShadow = FoliageType->bCastContactShadow;
			bNeedsMarkRenderStateDirty = true;
			bNeedsInvalidateLightingCache = true;
		}
		if (Component->ShadowCacheInvalidationBehavior != FoliageType->ShadowCacheInvalidationBehavior)
		{
			Component->ShadowCacheInvalidationBehavior = FoliageType->ShadowCacheInvalidationBehavior;
			bNeedsMarkRenderStateDirty = true;
		}
		if (Component->RuntimeVirtualTextures != FoliageType->RuntimeVirtualTextures)
		{
			Component->RuntimeVirtualTextures = FoliageType->RuntimeVirtualTextures;
			bNeedsMarkRenderStateDirty = true;
		}
		if (Component->VirtualTextureRenderPassType != FoliageType->VirtualTextureRenderPassType)
		{
			Component->VirtualTextureRenderPassType = FoliageType->VirtualTextureRenderPassType;
			bNeedsMarkRenderStateDirty = true;
		}
		if (Component->VirtualTextureCullMips != FoliageType->VirtualTextureCullMips)
		{
			Component->VirtualTextureCullMips = static_cast<int8>(FoliageType->VirtualTextureCullMips);
			bNeedsMarkRenderStateDirty = true;
		}
		if (Component->TranslucencySortPriority != FoliageType->TranslucencySortPriority)
		{
			Component->TranslucencySortPriority = FoliageType->TranslucencySortPriority;
			bNeedsMarkRenderStateDirty = true;
		}
		if (Component->bAffectDynamicIndirectLighting != FoliageType->bAffectDynamicIndirectLighting)
		{
			Component->bAffectDynamicIndirectLighting = FoliageType->bAffectDynamicIndirectLighting;
			bNeedsMarkRenderStateDirty = true;
			bNeedsInvalidateLightingCache = true;
		}
		if (Component->bAffectDistanceFieldLighting != FoliageType->bAffectDistanceFieldLighting)
		{
			Component->bAffectDistanceFieldLighting = FoliageType->bAffectDistanceFieldLighting;
			bNeedsMarkRenderStateDirty = true;
			bNeedsInvalidateLightingCache = true;
		}
		if (Component->bCastShadowAsTwoSided != FoliageType->bCastShadowAsTwoSided)
		{
			Component->bCastShadowAsTwoSided = FoliageType->bCastShadowAsTwoSided;
			bNeedsMarkRenderStateDirty = true;
			bNeedsInvalidateLightingCache = true;
		}
		if (Component->bReceivesDecals != FoliageType->bReceivesDecals)
		{
			Component->bReceivesDecals = FoliageType->bReceivesDecals;
			bNeedsMarkRenderStateDirty = true;
			bNeedsInvalidateLightingCache = true;
		}
		if (Component->bOverrideLightMapRes != FoliageType->bOverrideLightMapRes)
		{
			Component->bOverrideLightMapRes = FoliageType->bOverrideLightMapRes;
			bNeedsMarkRenderStateDirty = true;
			bNeedsInvalidateLightingCache = true;
		}
		if (Component->OverriddenLightMapRes != FoliageType->OverriddenLightMapRes)
		{
			Component->OverriddenLightMapRes = FoliageType->OverriddenLightMapRes;
			bNeedsMarkRenderStateDirty = true;
			bNeedsInvalidateLightingCache = true;
		}
		if (Component->GetLightmapType() != FoliageType->LightmapType)
		{
			Component->SetLightmapType(FoliageType->LightmapType);
			bNeedsMarkRenderStateDirty = true;
			bNeedsInvalidateLightingCache = true;
		}
		if (Component->bUseAsOccluder != FoliageType->bUseAsOccluder)
		{
			Component->bUseAsOccluder = FoliageType->bUseAsOccluder;
			bNeedsMarkRenderStateDirty = true;
		}
		if (Component->bVisibleInRayTracing != FoliageType->bVisibleInRayTracing)
		{
			Component->bVisibleInRayTracing = FoliageType->bVisibleInRayTracing;
			bNeedsMarkRenderStateDirty = true;
		}
		if (Component->bVisibleInReflections != FoliageType->bVisibleInReflections)
		{
			Component->bVisibleInReflections = FoliageType->bVisibleInReflections;
			bNeedsMarkRenderStateDirty = true;
		}
		if (Component->bEvaluateWorldPositionOffset != FoliageType->bEvaluateWorldPositionOffset)
		{
			Component->bEvaluateWorldPositionOffset = FoliageType->bEvaluateWorldPositionOffset;
			bNeedsMarkRenderStateDirty = true;
		}
		if (Component->bEvaluateWorldPositionOffsetInRayTracing != FoliageType->bEvaluateWorldPositionOffset)
		{
			Component->bEvaluateWorldPositionOffsetInRayTracing = FoliageType->bEvaluateWorldPositionOffset;
			bNeedsMarkRenderStateDirty = true;
		}
		if (Component->WorldPositionOffsetDisableDistance != FoliageType->WorldPositionOffsetDisableDistance)
		{
			Component->WorldPositionOffsetDisableDistance = FoliageType->WorldPositionOffsetDisableDistance;
			bNeedsMarkRenderStateDirty = true;
		}

		if(Component->NanitePixelProgrammableDistance != FoliageType->NanitePixelProgrammableDistance)
		{
			Component->NanitePixelProgrammableDistance = FoliageType->NanitePixelProgrammableDistance;
			bNeedsMarkRenderStateDirty = true;
		}


		if (Component->bEnableDensityScaling != FoliageType->bEnableDensityScaling)
		{
			Component->bEnableDensityScaling = FoliageType->bEnableDensityScaling;

			Component->UpdateDensityScaling();

			bNeedsMarkRenderStateDirty = true;
		}

		if (GetLightingChannelMaskForStruct(Component->LightingChannels) != GetLightingChannelMaskForStruct(FoliageType->LightingChannels))
		{
			Component->LightingChannels = FoliageType->LightingChannels;
			bNeedsMarkRenderStateDirty = true;
		}

		UFoliageInstancedStaticMeshComponent* FoliageComponent = Cast<UFoliageInstancedStaticMeshComponent>(Component);

		if (FoliageComponent && FoliageComponent->FoliageHiddenEditorViews != InSettings->HiddenEditorViews)
		{
			FoliageComponent->FoliageHiddenEditorViews = InSettings->HiddenEditorViews;
			bNeedsMarkRenderStateDirty = true;
		}

		if (Component->bRenderCustomDepth != FoliageType->bRenderCustomDepth)
		{
			Component->bRenderCustomDepth = FoliageType->bRenderCustomDepth;
			bNeedsMarkRenderStateDirty = true;
		}

		if (Component->CustomDepthStencilWriteMask != FoliageType->CustomDepthStencilWriteMask)
		{
			Component->CustomDepthStencilWriteMask = FoliageType->CustomDepthStencilWriteMask;
			bNeedsMarkRenderStateDirty = true;
		}

		if (Component->CustomDepthStencilValue != FoliageType->CustomDepthStencilValue)
		{
			Component->CustomDepthStencilValue = FoliageType->CustomDepthStencilValue;
			bNeedsMarkRenderStateDirty = true;
		}

		if (Component->bEnableAutoLODGeneration != FoliageType->bIncludeInHLOD)
		{
			Component->bEnableAutoLODGeneration = FoliageType->bIncludeInHLOD;
			bNeedsMarkRenderStateDirty = true;
		}

		const UFoliageType_InstancedStaticMesh* FoliageType_ISM = Cast<UFoliageType_InstancedStaticMesh>(FoliageType);
		if (FoliageType_ISM)
		{
			// Check override materials
			if (Component->OverrideMaterials.Num() != FoliageType_ISM->OverrideMaterials.Num())
			{
				Component->OverrideMaterials = FoliageType_ISM->OverrideMaterials;
				bNeedsMarkRenderStateDirty = true;
				bNeedsInvalidateLightingCache = true;
			}
			else
			{
				for (int32 Index = 0; Index < FoliageType_ISM->OverrideMaterials.Num(); Index++)
				{
					if (Component->OverrideMaterials[Index] != FoliageType_ISM->OverrideMaterials[Index])
					{
						Component->OverrideMaterials = FoliageType_ISM->OverrideMaterials;
						bNeedsMarkRenderStateDirty = true;
						bNeedsInvalidateLightingCache = true;
						break;
					}
				}
			}
		}

		Component->BodyInstance.CopyBodyInstancePropertiesFrom(&FoliageType->BodyInstance);

		Component->SetCustomNavigableGeometry(FoliageType->CustomNavigableGeometry);

		if (bNeedsInvalidateLightingCache)
		{
			Component->InvalidateLightingCache();
		}

		if (bNeedsMarkRenderStateDirty)
		{
			Component->MarkRenderStateDirty();
		}
	}
}

void FFoliageStaticMesh::Reapply(const UFoliageType* FoliageType)
{
	if (Component)
	{
		// clear the transactional flag if it was set prior to deleting the actor
		Component->ClearFlags(RF_Transactional);

		const bool bWasRegistered = Component->IsRegistered();
		Component->UnregisterComponent();

		Component->bAutoRebuildTreeOnInstanceChanges = false;

		// Update the instance transforms, attempting to re-use existing instances where possible
		const int32 NumInstancesToUpdate = FMath::Min(Component->GetInstanceCount(), Info->Instances.Num());
		if (Component->GetInstanceCount() < Info->Instances.Num())
		{
			// Add any missing instances
			TArray<FTransform> InstanceTransforms;
			Algo::Transform(MakeArrayView(Info->Instances.GetData() + Component->GetInstanceCount(), Info->Instances.Num() - Component->GetInstanceCount()), InstanceTransforms, &FFoliageInstance::GetInstanceWorldTransform);
			Component->AddInstances(InstanceTransforms, /*bShouldReturnIndices*/false, /*bWorldSpace*/true);
		}
		else if (Component->GetInstanceCount() > Info->Instances.Num())
		{
			// Remove any redundant instances
			TArray<int32> InstancesToRemove;
			for (int32 InstanceIndex = Info->Instances.Num(); InstanceIndex < Component->GetInstanceCount(); ++InstanceIndex)
			{
				InstancesToRemove.Add(InstanceIndex);
			}
			Component->RemoveInstances(InstancesToRemove);
		}
		check(Component->GetInstanceCount() == Info->Instances.Num());
		if (NumInstancesToUpdate > 0)
		{
			// Update any existing instances
			TArray<FTransform> InstanceTransforms;
			Algo::Transform(MakeArrayView(Info->Instances.GetData(), NumInstancesToUpdate), InstanceTransforms, &FFoliageInstance::GetInstanceWorldTransform);
			Component->BatchUpdateInstancesTransforms(0, InstanceTransforms, /*bWorldSpace*/true, /*bMarkRenderStateDirty*/true, /*bTeleport*/true);
		}

		Component->bAutoRebuildTreeOnInstanceChanges = true;
		Component->BuildTreeIfOutdated(true, true);

		// Update the selection state of the instances
		Component->ClearInstanceSelection();
		if (Info->SelectedIndices.Num())
		{
			for (int32 i : Info->SelectedIndices)
			{
				Component->SelectInstance(true, i, 1);
			}
		}

		if (bWasRegistered)
		{
			Component->RegisterComponent();
		}
	}
}

FFoliageInstanceId FFoliageStaticMesh::SMInstanceIdToFoliageInstanceId(const FSMInstanceId& InstanceId) const
{
	check(InstanceId.ISMComponent == Component);
	return FFoliageInstanceId{ Info, InstanceId.InstanceIndex };
}

TArray<FFoliageInstanceId> FFoliageStaticMesh::SMInstanceIdsToFoliageInstanceIds(TArrayView<const FSMInstanceId> InstanceIds) const
{
	TArray<FFoliageInstanceId> FoliageInstanceIds;
	FoliageInstanceIds.Reserve(InstanceIds.Num());
	for (const FSMInstanceId& InstanceId : InstanceIds)
	{
		FoliageInstanceIds.Add(SMInstanceIdToFoliageInstanceId(InstanceId));
	}
	return FoliageInstanceIds;
}

AInstancedFoliageActor* FFoliageImpl::GetIFA() const
{
	return Info->IFA;
}

#endif // WITH_EDITOR

bool FFoliageStaticMesh::CanEditSMInstance(const FSMInstanceId& InstanceId) const
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = SMInstanceIdToFoliageInstanceId(InstanceId);
	return GetIFA()->CanEditFoliageInstance(FoliageInstanceId);
#else
	return false;
#endif
}

bool FFoliageStaticMesh::CanMoveSMInstance(const FSMInstanceId& InstanceId, const ETypedElementWorldType WorldType) const
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = SMInstanceIdToFoliageInstanceId(InstanceId);
	return GetIFA()->CanMoveFoliageInstance(FoliageInstanceId, WorldType);
#else
	return false;
#endif
}

bool FFoliageStaticMesh::GetSMInstanceTransform(const FSMInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace) const
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = SMInstanceIdToFoliageInstanceId(InstanceId);
	return GetIFA()->GetFoliageInstanceTransform(FoliageInstanceId, OutInstanceTransform, bWorldSpace);
#else
	return false;
#endif
}

bool FFoliageStaticMesh::SetSMInstanceTransform(const FSMInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = SMInstanceIdToFoliageInstanceId(InstanceId);
	return GetIFA()->SetFoliageInstanceTransform(FoliageInstanceId, InstanceTransform, bWorldSpace, bTeleport);
#else
	return false;
#endif
}

void FFoliageStaticMesh::NotifySMInstanceMovementStarted(const FSMInstanceId& InstanceId)
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = SMInstanceIdToFoliageInstanceId(InstanceId);
	GetIFA()->NotifyFoliageInstanceMovementStarted(FoliageInstanceId);
#endif
}

void FFoliageStaticMesh::NotifySMInstanceMovementOngoing(const FSMInstanceId& InstanceId)
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = SMInstanceIdToFoliageInstanceId(InstanceId);
	GetIFA()->NotifyFoliageInstanceMovementOngoing(FoliageInstanceId);
#endif
}

void FFoliageStaticMesh::NotifySMInstanceMovementEnded(const FSMInstanceId& InstanceId)
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = SMInstanceIdToFoliageInstanceId(InstanceId);
	GetIFA()->NotifyFoliageInstanceMovementEnded(FoliageInstanceId);
#endif
}

void FFoliageStaticMesh::NotifySMInstanceSelectionChanged(const FSMInstanceId& InstanceId, const bool bIsSelected)
{
#if WITH_EDITOR
	const FFoliageInstanceId FoliageInstanceId = SMInstanceIdToFoliageInstanceId(InstanceId);
	GetIFA()->NotifyFoliageInstanceSelectionChanged(FoliageInstanceId, bIsSelected);
#endif
}

bool FFoliageStaticMesh::DeleteSMInstances(TArrayView<const FSMInstanceId> InstanceIds)
{
#if WITH_EDITOR
	const TArray<FFoliageInstanceId> FoliageInstanceIds = SMInstanceIdsToFoliageInstanceIds(InstanceIds);
	return GetIFA()->DeleteFoliageInstances(FoliageInstanceIds);
#else
	return false;
#endif
}

bool FFoliageStaticMesh::DuplicateSMInstances(TArrayView<const FSMInstanceId> InstanceIds, TArray<FSMInstanceId>& OutNewInstanceIds)
{
#if WITH_EDITOR
	const TArray<FFoliageInstanceId> FoliageInstanceIds = SMInstanceIdsToFoliageInstanceIds(InstanceIds);

	TArray<FFoliageInstanceId> NewFoliageInstanceIds;
	const bool bDidDuplicate = GetIFA()->DuplicateFoliageInstances(FoliageInstanceIds, NewFoliageInstanceIds);

	OutNewInstanceIds.Reset(NewFoliageInstanceIds.Num());
	for (const FFoliageInstanceId& NewFoliageInstanceId : NewFoliageInstanceIds)
	{
		OutNewInstanceIds.Add(FSMInstanceId{ NewFoliageInstanceId.Info->GetComponent(), NewFoliageInstanceId.Index });
	}

	return bDidDuplicate;
#else
	return false;
#endif
}

//
// FFoliageInfo
//
FFoliageInfo::FFoliageInfo()
	: Type(EFoliageImplType::StaticMesh)
#if WITH_EDITORONLY_DATA
	, IFA(nullptr)
	  // This will cover cases of running in editor or as editor with -game.  We do not want this during cooking.
	, InstanceHash(GIsEditor || IsRunningGame() ? new FFoliageInstanceHash() : nullptr)
#endif
{ }

FFoliageInfo::~FFoliageInfo()
{ }

UHierarchicalInstancedStaticMeshComponent* FFoliageInfo::GetComponent() const
{
	if (Type == EFoliageImplType::StaticMesh && Implementation.IsValid())
	{
		FFoliageStaticMesh* FoliageStaticMesh = StaticCast<FFoliageStaticMesh*>(Implementation.Get());
		return FoliageStaticMesh->Component;
	}

	return nullptr;
}

void FFoliageInfo::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	if (Implementation.IsValid())
	{
		Implementation->AddReferencedObjects(InThis, Collector);
	}
}

void FFoliageInfo::PostSerialize(FArchive& Ar)
{
	if (Implementation)
	{
		Implementation->PostSerialize(Ar);
	}
}

void FFoliageInfo::PostLoad()
{
	if (Implementation)
	{
		Implementation->PostLoad();
	}
}

void FFoliageInfo::CreateImplementation(EFoliageImplType InType)
{
	check(InType != EFoliageImplType::Unknown);
	check(!Implementation.IsValid());
	// Change Impl based on InType param
	Type = InType;
	
	if (Type == EFoliageImplType::StaticMesh)
	{
		Implementation.Reset(new FFoliageStaticMesh(this, nullptr));
	}
	else if (Type == EFoliageImplType::Actor)
	{
		Implementation.Reset(new FFoliageActor(this));
	}
	else if (Type == EFoliageImplType::ISMActor)
	{
		Implementation.Reset(new FFoliageISMActor(this));
	}
}

int32 FFoliageInfo::GetOverlappingSphereCount(const FSphere& Sphere) const
{
	if (Implementation.IsValid())
	{
		return Implementation->GetOverlappingSphereCount(Sphere);
	}

	return 0;
}

int32 FFoliageInfo::GetOverlappingBoxCount(const FBox& Box) const
{
	if (Implementation.IsValid())
	{
		return Implementation->GetOverlappingBoxCount(Box);
	}

	return 0;
}

void FFoliageInfo::GetOverlappingBoxTransforms(const FBox& Box, TArray<FTransform>& OutTransforms) const
{
	if (Implementation.IsValid())
	{
		Implementation->GetOverlappingBoxTransforms(Box, OutTransforms);
	}
}

void FFoliageInfo::GetOverlappingMeshCount(const FSphere& Sphere, TMap<UStaticMesh*, int32>& OutCounts) const
{
	if (Implementation.IsValid())
	{
		Implementation->GetOverlappingMeshCount(Sphere, OutCounts);
	}
}

#if WITH_EDITOR
EFoliageImplType FFoliageInfo::GetImplementationType(const UFoliageType* FoliageType) const
{
	if (FoliageType->IsA<UFoliageType_InstancedStaticMesh>())
	{
		return EFoliageImplType::StaticMesh;
	}
	else if (FoliageType->IsA<UFoliageType_Actor>())
	{
		const UFoliageType_Actor* ActorFoliageType = Cast<UFoliageType_Actor>(FoliageType);
		if (ActorFoliageType->bStaticMeshOnly)
		{
			return EFoliageImplType::ISMActor;
		}
		else
		{
			return EFoliageImplType::Actor;
		}
	}

	return EFoliageImplType::Unknown;
}

void FFoliageInfo::CreateImplementation(const UFoliageType* FoliageType)
{
	check(!Implementation.IsValid());
	
	CreateImplementation(GetImplementationType(FoliageType));
}

void FFoliageInfo::Initialize(const UFoliageType* FoliageType)
{
	check(!IsInitialized());
	check(Implementation.IsValid());
	Implementation->Initialize(FoliageType);
}

void FFoliageInfo::Uninitialize()
{
	check(IsInitialized());
	FoliageTypeUpdateGuid.Invalidate();
	Implementation->Uninitialize();
}

bool FFoliageInfo::IsInitialized() const
{
	return Implementation.IsValid() && Implementation->IsInitialized();
}

void FFoliageInfo::NotifyFoliageTypeWillChange(UFoliageType* FoliageType)
{
	Implementation->NotifyFoliageTypeWillChange(FoliageType);
}

void FFoliageInfo::NotifyFoliageTypeChanged(UFoliageType* FoliageType, bool bSourceChanged)
{
	FoliageTypeUpdateGuid = FoliageType->UpdateGuid;
	if (Implementation->NotifyFoliageTypeChanged(FoliageType, bSourceChanged))
	{
		ReallocateClusters(FoliageType);
	}
}

void FFoliageInfo::CheckValid()
{
#if DO_FOLIAGE_CHECK
	int32 ClusterTotal = 0;
	int32 ComponentTotal = 0;

	check(Instances.Num() == Implementation->GetInstanceCount());
		
	InstanceHash->CheckInstanceCount(Instances.Num());

	int32 ComponentHashTotal = 0;
	for (const auto& Pair : ComponentHash)
	{
		ComponentHashTotal += Pair.Value.Num();
	}
	check(ComponentHashTotal == Instances.Num());

#if FOLIAGE_CHECK_TRANSFORM
	// Check transforms match up with editor data
	int32 MismatchCount = 0;
	for (int32 i = 0; i < Instances.Num(); ++i)
	{
		FTransform InstanceToWorldEd = Instances[i].GetInstanceWorldTransform();
		FTransform InstanceToWorldImpl = Implementation->GetInstanceWorldTransform(i);

		if (!InstanceToWorldEd.Equals(InstanceToWorldImpl))
		{
			MismatchCount++;
		}
	}
		
	if (MismatchCount != 0)
	{
		UE_LOGF(LogInstancedFoliage, Log, "transform mismatch: %d", MismatchCount);
	}
#endif

#endif
}

void FFoliageInfo::ForEachSMInstance(TFunctionRef<bool(FSMInstanceId)> Callback) const
{
	if (Implementation)
	{
		Implementation->ForEachSMInstance(Callback);
	}
}

void FFoliageInfo::ForEachSMInstance(int32 InstanceIndex, TFunctionRef<bool(FSMInstanceId)> Callback) const
{
	if (Implementation)
	{
		Implementation->ForEachSMInstance(InstanceIndex, Callback);
	}
}

/*
*/
void FFoliageInfo::ClearSelection()
{
	if (Instances.Num() > 0)
	{
		Implementation->ClearSelection(SelectedIndices);
		SelectedIndices.Empty();
	}
}

void FFoliageInfo::SetRandomSeed(int32 seed)
{
	if (Type == EFoliageImplType::StaticMesh)
	{
		FFoliageStaticMesh* FoliageStaticMesh = StaticCast<FFoliageStaticMesh*>(Implementation.Get());
		FoliageStaticMesh->Component->InstancingRandomSeed = 1;
	}
}

void FFoliageInfo::SetInstanceWorldTransform(int32 InstanceIndex, const FTransform& Transform, bool bTeleport)
{
	Implementation->SetInstanceWorldTransform(InstanceIndex, Transform, bTeleport);
}

void FFoliageInfo::AddInstanceImpl(const FFoliageInstance& InNewInstance, FFoliageInfo::FAddImplementationFunc ImplementationFunc)
{
	// Add the instance taking either a free slot or adding a new item.
	int32 InstanceIndex = Instances.Add(InNewInstance);
	FFoliageInstance& AddedInstance = Instances[InstanceIndex];

	AddedInstance.BaseId = IFA->InstanceBaseCache.AddInstanceBaseId(ShouldAttachToBaseComponent()? InNewInstance.BaseComponent : nullptr);
	if (AddedInstance.BaseId == FFoliageInstanceBaseCache::InvalidBaseId)
	{
		AddedInstance.BaseComponent = nullptr;
	}

	// Add the instance to the hash
	AddToBaseHash(InstanceIndex);
	InstanceHash->InsertInstance(AddedInstance.Location, InstanceIndex);

	// Add the instance to the component
	ImplementationFunc(Implementation.Get(), IFA, AddedInstance);
}

void FFoliageInfo::AddInstances(const UFoliageType* InSettings, const TArray<const FFoliageInstance*>& InNewInstances)
{
	AddInstancesImpl(InSettings, InNewInstances, [](FFoliageImpl* Impl, AInstancedFoliageActor* LocalIFA, const FFoliageInstance& LocalInstance) { Impl->AddInstance(LocalInstance); });
}

void FFoliageInfo::ReserveAdditionalInstances(const UFoliageType* InSettings, uint32 ReserveNum)
{
	Instances.Reserve(Instances.Num() + ReserveNum);
	Implementation->PreAddInstances(InSettings, ReserveNum);
}

void FFoliageInfo::AddInstancesImpl(const UFoliageType* InSettings, const TArray<const FFoliageInstance*>& InNewInstances, FFoliageInfo::FAddImplementationFunc ImplementationFunc)
{
	SCOPE_CYCLE_COUNTER(STAT_FoliageAddInstance);

	IFA->Modify();

	Implementation->PreAddInstances(InSettings, InNewInstances.Num());

	Implementation->BeginUpdate();

	Instances.Reserve(Instances.Num() + InNewInstances.Num());

	for (const FFoliageInstance* Instance : InNewInstances)
	{
		AddInstanceImpl(*Instance, ImplementationFunc);
	}

	CheckValid();

	Implementation->EndUpdate();
}

void FFoliageInfo::AddInstance(const UFoliageType* InSettings, const FFoliageInstance& InNewInstance, UActorComponent* InBaseComponent)
{
	FFoliageInstance Instance = InNewInstance;
	Instance.BaseId = IFA->InstanceBaseCache.AddInstanceBaseId(ShouldAttachToBaseComponent() ? InBaseComponent : nullptr);
	if (Instance.BaseId == FFoliageInstanceBaseCache::InvalidBaseId)
	{
		Instance.BaseComponent = nullptr;
	}
	AddInstance(InSettings, Instance);
}

void FFoliageInfo::AddInstance(const UFoliageType* InSettings, const FFoliageInstance& InNewInstance)
{
	AddInstances(InSettings, { &InNewInstance });
}

void FFoliageInfo::MoveInstances(AInstancedFoliageActor* InToIFA, const TSet<int32>& InInstancesToMove, bool bKeepSelection)
{
	const UFoliageType* FoliageType = IFA->FindFoliageType(this);
	check(FoliageType);

	IFA->Modify();
	
	FFoliageInfo* OutFoliageInfo = nullptr;
	UFoliageType* ToFoliageType = nullptr;
	if (InToIFA)
	{
		InToIFA->Modify();
		ToFoliageType = InToIFA->AddFoliageType(FoliageType, &OutFoliageInfo);
	}

	TArray<int32> NewSelectedIndices;
	
	struct FFoliageMoveInstance : FFoliageInstance
	{
		FFoliageMoveInstance(const FFoliageInstance& InInstance, UActorComponent* InBaseComponent)
			: FFoliageInstance(InInstance)
		{
			BaseComponent = InBaseComponent;
		}
		UObject* InstanceImplementation = nullptr;
	};

	TArray<int32> InstancesToMove = InInstancesToMove.Array();
	TMap<int32, FFoliageMoveInstance> MoveData;
	MoveData.Reserve(InInstancesToMove.Num());
	
	for (int32 InstanceIndex : InInstancesToMove)
	{
		MoveData.Add(InstanceIndex, FFoliageMoveInstance(Instances[InstanceIndex], IFA->GetBaseComponentFromBaseId(Instances[InstanceIndex].BaseId)));
		if (bKeepSelection && OutFoliageInfo && SelectedIndices.Contains(InstanceIndex))
		{
			NewSelectedIndices.Add(OutFoliageInfo->Instances.Num() + MoveData.Num() - 1);
		}
	}

	RemoveInstancesImpl(InstancesToMove, true, [&MoveData](FFoliageImpl* Impl, int32 Index)
	{
		Impl->MoveInstance(Index, MoveData[Index].InstanceImplementation);
	});

	TArray<const FFoliageInstance*> NewInstances;
	NewInstances.Reserve(MoveData.Num());
	Algo::Transform(MoveData, NewInstances, [](const TPair<int32, FFoliageMoveInstance>& Pair)
	{
		return &Pair.Value;
	});
	
	if (InToIFA)
	{
		int32 AddedIndex = 0;
		
		OutFoliageInfo->AddInstancesImpl(ToFoliageType, NewInstances, [&](FFoliageImpl* Impl, AInstancedFoliageActor* LocalIFA, const FFoliageInstance& LocalInstance)
		{
			const FFoliageMoveInstance* MoveInstance = StaticCast<const FFoliageMoveInstance*>(NewInstances[AddedIndex]);
			Impl->AddExistingInstance(LocalInstance, MoveInstance->InstanceImplementation);
			AddedIndex++;
		});

		OutFoliageInfo->Refresh(true, true);

		// Select if needed
		if (NewSelectedIndices.Num())
		{
			OutFoliageInfo->SelectInstances(true, NewSelectedIndices);
		}
	}
}

void FFoliageInfo::RemoveInstances(TArrayView<const int32> InInstancesToRemove, bool RebuildFoliageTree)
{
	RemoveInstancesImpl(InInstancesToRemove, RebuildFoliageTree, [](FFoliageImpl* Impl, int32 Index) { Impl->RemoveInstance(Index); });
}

void FFoliageInfo::RemoveInstancesImpl(TArrayView<const int32> InInstancesToRemove, bool RebuildFoliageTree, FFoliageInfo::FRemoveImplementationFunc ImplementationFunc)
{
	SCOPE_CYCLE_COUNTER(STAT_FoliageRemoveInstance);

	if (InInstancesToRemove.Num() <= 0)
	{
		return;
	}

	check(IsInitialized());
	IFA->Modify();

	Implementation->BeginUpdate();

	TSet<int32> InstancesToRemove;
	InstancesToRemove.Reserve(InInstancesToRemove.Num());
	for (int32 InstanceToRemove : InInstancesToRemove)
	{
		InstancesToRemove.Add(InstanceToRemove);
	}

	while (InstancesToRemove.Num() > 0)
	{
		// Get an item from the set for processing
		auto It = InstancesToRemove.CreateConstIterator();
		int32 InstanceIndex = *It;
		int32 InstanceIndexToRemove = InstanceIndex;

		FFoliageInstance& Instance = Instances[InstanceIndex];

		// remove from hash
		RemoveFromBaseHash(InstanceIndex);
		InstanceHash->RemoveInstance(Instance.Location, InstanceIndex);

		// remove from the component
		ImplementationFunc(Implementation.Get(), InstanceIndex);

		// Remove it from the selection.
		SelectedIndices.Remove(InstanceIndex);

		// remove from instances array
		Instances.RemoveAtSwap(InstanceIndex, EAllowShrinking::No);

		// update hashes for swapped instance
		if (InstanceIndex != Instances.Num() && Instances.Num() > 0)
		{
			// Instance hash
			FFoliageInstance& SwappedInstance = Instances[InstanceIndex];
			InstanceHash->RemoveInstance(SwappedInstance.Location, Instances.Num());
			InstanceHash->InsertInstance(SwappedInstance.Location, InstanceIndex);

			// Component hash
			auto* InstanceSet = ComponentHash.Find(SwappedInstance.BaseId);
			if (InstanceSet)
			{
				InstanceSet->Remove(Instances.Num());
				InstanceSet->Add(InstanceIndex);
			}

			// Selection
			if (SelectedIndices.Contains(Instances.Num()))
			{
				SelectedIndices.Remove(Instances.Num());
				SelectedIndices.Add(InstanceIndex);
			}

			// Removal list
			if (InstancesToRemove.Contains(Instances.Num()))
			{
				// The item from the end of the array that we swapped in to InstanceIndex is also on the list to remove.
				// Remove the item at the end of the array and leave InstanceIndex in the removal list.
				InstanceIndexToRemove = Instances.Num();
			}
		}

		// Remove the removed item from the removal list
		InstancesToRemove.Remove(InstanceIndexToRemove);
	}

	Instances.Shrink();
		
	Implementation->EndUpdate();
		
	if (RebuildFoliageTree)
	{
		Refresh(true, true);
	}

	CheckValid();
}

void FFoliageInfo::PreMoveInstances(TArrayView<const int32> InInstancesMoved)
{
	// Remove instances from the hash
	for (int32 InstanceIndex : InInstancesMoved)
	{
		checkf(!MovingInstances.Contains(InstanceIndex), TEXT("PreMoveInstances has already been called for index %d!"), InstanceIndex);
		MovingInstances.Add(InstanceIndex);

		const FFoliageInstance& Instance = Instances[InstanceIndex];
		InstanceHash->RemoveInstance(Instance.Location, InstanceIndex);
	}

	Implementation->PreMoveInstances(InInstancesMoved);
}

void FFoliageInfo::PostUpdateInstances(TArrayView<const int32> InInstancesUpdated, bool bReAddToHash, bool InUpdateSelection)
{
	if (InInstancesUpdated.Num())
	{
		TSet<int32> UpdateSelectedIndices;
		UpdateSelectedIndices.Reserve(InInstancesUpdated.Num());
		for (int32 InstanceIndex : InInstancesUpdated)
		{
			const FFoliageInstance& Instance = Instances[InstanceIndex];

			FTransform InstanceToWorld = Instance.GetInstanceWorldTransform();

			Implementation->SetInstanceWorldTransform(InstanceIndex, InstanceToWorld, true);

			// Re-add instance to the hash if requested
			if (bReAddToHash)
			{
				InstanceHash->InsertInstance(Instance.Location, InstanceIndex);
			}

			// Reselect the instance to update the render update to include selection as by default it gets removed
			if (InUpdateSelection)
			{
				UpdateSelectedIndices.Add(InstanceIndex);
			}
		}

		if (UpdateSelectedIndices.Num() > 0)
		{
			Implementation->SelectInstances(true, UpdateSelectedIndices);
		}

		Implementation->PostUpdateInstances();
	}
}

void FFoliageInfo::PostMoveInstances(TArrayView<const int32> InInstancesMoved, bool bFinished)
{
	PostUpdateInstances(InInstancesMoved, /*bReAddToHash*/true, /*UpdateSelection*/true);
	Implementation->PostMoveInstances(InInstancesMoved, bFinished);

	for (int32 InstanceIndex : InInstancesMoved)
	{
		checkf(MovingInstances.Contains(InstanceIndex), TEXT("PostMoveInstances has already been called for index %d, or a call to PreMoveInstances is missing!"), InstanceIndex);
		MovingInstances.Remove(InstanceIndex);
	}
}

void FFoliageInfo::DuplicateInstances(UFoliageType* InSettings, TArrayView<const int32> InInstancesToDuplicate)
{
	Implementation->BeginUpdate();

	for (int32 InstanceIndex : InInstancesToDuplicate)
	{
		const FFoliageInstance TempInstance = Instances[InstanceIndex];
		AddInstance(InSettings, TempInstance);
	}

	Implementation->EndUpdate();
	Refresh(true, true);
}

/* Get the number of placed instances */
int32 FFoliageInfo::GetPlacedInstanceCount() const
{
	int32 PlacedInstanceCount = 0;

	for (int32 i = 0; i < Instances.Num(); ++i)
	{
		if (!Instances[i].ProceduralGuid.IsValid())
		{
			++PlacedInstanceCount;
		}
	}

	return PlacedInstanceCount;
}

void FFoliageInfo::AddToBaseHash(int32 InstanceIndex)
{
	FFoliageInstance& Instance = Instances[InstanceIndex];
	ComponentHash.FindOrAdd(Instance.BaseId).Add(InstanceIndex);
}

void FFoliageInfo::RemoveFromBaseHash(int32 InstanceIndex)
{
	FFoliageInstance& Instance = Instances[InstanceIndex];

	// Remove current base link
	auto* InstanceSet = ComponentHash.Find(Instance.BaseId);
	if (InstanceSet)
	{
		InstanceSet->Remove(InstanceIndex);
		if (InstanceSet->Num() == 0)
		{
			// Remove the component from the component hash if this is the last instance.
			ComponentHash.Remove(Instance.BaseId);
		}
	}
}

void FFoliageInfo::RecomputeHash()
{
	ComponentHash.Empty();
	InstanceHash->Empty();
	for (int32 InstanceIdx = 0; InstanceIdx < Instances.Num(); InstanceIdx++)
	{
		// Invalidate base if we aren't supposed to be attached.
		if (!ShouldAttachToBaseComponent())
		{
			Instances[InstanceIdx].BaseId = FFoliageInstanceBaseCache::InvalidBaseId;
			Instances[InstanceIdx].BaseComponent = nullptr;
		}
		AddToBaseHash(InstanceIdx);
		InstanceHash->InsertInstance(Instances[InstanceIdx].Location, InstanceIdx);
	}
}

// Destroy existing clusters and reassign all instances to new clusters
void FFoliageInfo::ReallocateClusters(UFoliageType* InSettings)
{
	// In case Foliage Type Changed recreate implementation
	if (Implementation.IsValid() && Implementation->IsInitialized())
	{
		Implementation->Uninitialize();
	}
	Implementation.Reset();
	CreateImplementation(InSettings);
	
	// Remove everything
	TArray<FFoliageInstance> OldInstances;
	Exchange(Instances, OldInstances);
	InstanceHash->Empty();
	ComponentHash.Empty();
	SelectedIndices.Empty();

	// Copy the UpdateGuid from the foliage type
	FoliageTypeUpdateGuid = InSettings->UpdateGuid;

	// Filter instances to re-add
	TArray<const FFoliageInstance*> InstancesToReAdd;
	InstancesToReAdd.Reserve(OldInstances.Num());

	for (FFoliageInstance& Instance : OldInstances)
	{
		if ((Instance.Flags & FOLIAGE_InstanceDeleted) == 0)
		{
			InstancesToReAdd.Add(&Instance);
		}
	}

	// Finally, re-add the instances
	AddInstances(InSettings, InstancesToReAdd);
	
	Refresh(true, true);
}

void FFoliageInfo::GetInstancesInsideBounds(const FBox& Box, TArray<int32>& OutInstances) const
{
	auto TempInstances = InstanceHash->GetInstancesOverlappingBox(Box);
	for (int32 Idx : TempInstances)
	{
		if (Box.IsInsideOrOn(Instances[Idx].Location))
		{
			OutInstances.Add(Idx);
		}
	}
}

void FFoliageInfo::GetInstancesInsideSphere(const FSphere& Sphere, TArray<int32>& OutInstances) const
{
	auto TempInstances = InstanceHash->GetInstancesOverlappingBox(FBox::BuildAABB(Sphere.Center, FVector(Sphere.W)));
	for (int32 Idx : TempInstances)
	{
		if (FSphere(Instances[Idx].Location, 0.f).IsInside(Sphere))
		{
			OutInstances.Add(Idx);
		}
	}
}

void FFoliageInfo::GetInstanceAtLocation(const FVector& Location, int32& OutInstance, bool& bOutSucess) const
{
	auto TempInstances = InstanceHash->GetInstancesOverlappingBox(FBox::BuildAABB(Location, FVector(KINDA_SMALL_NUMBER)));

	FVector::FReal ShortestDistance = MAX_dbl;
	OutInstance = -1;

	for (int32 Idx : TempInstances)
	{
		FVector InstanceLocation = Instances[Idx].Location;
		FVector::FReal DistanceSquared = FVector::DistSquared(InstanceLocation, Location);
		if (DistanceSquared < ShortestDistance)
		{
			ShortestDistance = DistanceSquared;
			OutInstance = Idx;
		}
	}

	bOutSucess = OutInstance != -1;
}

// Returns whether or not there is are any instances overlapping the sphere specified
bool FFoliageInfo::CheckForOverlappingSphere(const FSphere& Sphere) const
{
	TArrayView<const FFoliageInstance> InstancesView = Instances;
	return InstanceHash->IsAnyInstanceInSphere([InstancesView](int32 Index) -> FVector { return InstancesView[Index].Location; }, Sphere.Center, Sphere.W);
}

// Returns whether or not there is are any instances overlapping the instance specified, excluding the set of instances provided
bool FFoliageInfo::CheckForOverlappingInstanceExcluding(int32 TestInstanceIdx, float Radius, TSet<int32>& ExcludeInstances) const
{
	FSphere Sphere(Instances[TestInstanceIdx].Location, Radius);

	auto TempInstances = InstanceHash->GetInstancesOverlappingBox(FBox::BuildAABB(Sphere.Center, FVector(Sphere.W)));
	for (int32 Idx : TempInstances)
	{
		if (Idx != TestInstanceIdx && !ExcludeInstances.Contains(Idx) && FSphere(Instances[Idx].Location, 0.f).IsInside(Sphere))
		{
			return true;
		}
	}
	return false;
}

void FFoliageInfo::SelectInstances(bool bSelect)
{
	if (Implementation->IsInitialized())
	{
		IFA->Modify(false);

		if (bSelect)
		{
			SelectedIndices.Reserve(Instances.Num());

			for (int32 i = 0; i < Instances.Num(); ++i)
			{
				SelectedIndices.Add(i);
			}

			Implementation->SelectAllInstances(true);
		}
		else
		{
			Implementation->ClearSelection(SelectedIndices);
			SelectedIndices.Empty();
		}
	}
}

FBox FFoliageInfo::GetSelectionBoundingBox() const
{
	if (Implementation->IsInitialized())
	{
		return Implementation->GetSelectionBoundingBox(SelectedIndices);
	}

	return FBox(EForceInit::ForceInit);
}

void FFoliageInfo::SelectInstances(bool bSelect, TArrayView<const int32> InInstances)
{
	if (InInstances.Num())
	{
		TSet<int32> ModifiedSelection;
		ModifiedSelection.Reserve(InInstances.Num());
		check(Implementation->IsInitialized());
		if (IFA)
		{
			IFA->Modify(false);
		}
		if (bSelect)
		{
			SelectedIndices.Reserve(InInstances.Num());

			for (int32 i : InInstances)
			{
				SelectedIndices.Add(i);
				ModifiedSelection.Add(i);
			}
		}
		else
		{
			for (int32 i : InInstances)
			{
				SelectedIndices.Remove(i);
				ModifiedSelection.Add(i);
			}
		}

		Implementation->SelectInstances(bSelect, ModifiedSelection);
	}
}

void FFoliageInfo::Refresh(bool Async, bool Force)
{
	check(Implementation.IsValid())
	Implementation->Refresh(Async, Force);
}

void FFoliageInfo::OnHiddenEditorViewMaskChanged(uint64 InHiddenEditorViews)
{
	Implementation->OnHiddenEditorViewMaskChanged(InHiddenEditorViews);
}

void FFoliageInfo::PreEditUndo(UFoliageType* FoliageType)
{
	Implementation->PreEditUndo(FoliageType);
}

void FFoliageInfo::PostEditUndo(UFoliageType* FoliageType)
{
	Implementation->PostEditUndo(this, FoliageType);
		
	// Regenerate instance hash
	// We regenerate it here instead of saving to transaction buffer to speed up modify operations
	InstanceHash->Empty();
	for (int32 InstanceIdx = 0; InstanceIdx < Instances.Num(); InstanceIdx++)
	{
		InstanceHash->InsertInstance(Instances[InstanceIdx].Location, InstanceIdx);
	}
}

void FFoliageInfo::EnterEditMode()
{
	Implementation->EnterEditMode();
}

void FFoliageInfo::ExitEditMode()
{
	Implementation->ExitEditMode();
}

void FFoliageInfo::RemoveBaseComponentOnInstances()
{
	for (int32 InstanceIdx = 0; InstanceIdx < Instances.Num(); InstanceIdx++)
	{
		RemoveFromBaseHash(InstanceIdx);
		Instances[InstanceIdx].BaseId = FFoliageInstanceBaseCache::InvalidBaseId;
		Instances[InstanceIdx].BaseComponent = nullptr;
		AddToBaseHash(InstanceIdx);
	}
}

void FFoliageInfo::IncludeActor(const UFoliageType* FoliageType, AActor* InActor)
{
	if (Type == EFoliageImplType::Actor)
	{
		if (FFoliageActor* FoliageActor = StaticCast<FFoliageActor*>(Implementation.Get()))
		{
			FFoliageInstance NewInstance;
			NewInstance.BaseComponent = nullptr;
			NewInstance.BaseId = FFoliageInstanceBaseCache::InvalidBaseId;

			NewInstance.DrawScale3D = (FVector3f)InActor->GetActorScale3D();
			FTransform Transform = InActor->GetTransform();
			NewInstance.Location = Transform.GetLocation();
			NewInstance.Rotation = FRotator(Transform.GetRotation());
			NewInstance.PreAlignRotation = NewInstance.Rotation;

			int32 Index = Instances.Add(NewInstance);
			InstanceHash->InsertInstance(NewInstance.Location, Index);

			if (FoliageActor->FindIndex(InActor) == INDEX_NONE)
			{
				FoliageActor->PreAddInstances(FoliageType, 1);
				FoliageActor->ActorInstances.Add(InActor);
				InActor->Modify();
				FFoliageHelper::SetIsOwnedByFoliage(InActor);
			}
		}
	}
}

void FFoliageInfo::ExcludeActors()
{
	if (Type == EFoliageImplType::Actor)
	{
		if (FFoliageActor* FoliageActor = StaticCast<FFoliageActor*>(Implementation.Get()))
		{
			SelectedIndices.Empty();
			Instances.Empty();
			InstanceHash->Empty();
			ComponentHash.Empty();
			for (AActor* Actor : FoliageActor->ActorInstances)
			{
				if (Actor != nullptr)
				{
					Actor->Modify();
					FFoliageHelper::SetIsOwnedByFoliage(Actor, false);
				}
			}
			FoliageActor->ActorInstances.Empty();
		}
	}
}

TArray<int32> FFoliageInfo::GetInstancesOverlappingBox(const FBox& Box) const
{
	return InstanceHash->GetInstancesOverlappingBox(Box);
}

FBox FFoliageInfo::GetApproximatedInstanceBounds() const
{
	return InstanceHash->GetBounds();
}
#endif	//WITH_EDITOR

//
// AInstancedFoliageActor
//
AInstancedFoliageActor::AInstancedFoliageActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetActorEnableCollision(true);
	PrimaryActorTick.bCanEverTick = false;
}

AInstancedFoliageActor* AInstancedFoliageActor::GetInstancedFoliageActorForCurrentLevel(const UWorld* InWorld, bool bCreateIfNone)
{
	return GetInstancedFoliageActorForLevel(InWorld->GetCurrentLevel(), bCreateIfNone);
}

AInstancedFoliageActor* AInstancedFoliageActor::GetInstancedFoliageActorForLevel(ULevel* InLevel, bool bCreateIfNone /* = false */)
{
	AInstancedFoliageActor* IFA = nullptr;
	if (InLevel)
	{
		//@todo_ow: This code path needs to be eliminated when in WorldPartition
		ensure(InLevel->GetWorld()->GetSubsystem<UActorPartitionSubsystem>()->IsLevelPartition());
		IFA = InLevel->InstancedFoliageActor.Get();

		if (!IFA && bCreateIfNone)
		{
			FActorSpawnParameters SpawnParams;
			SpawnParams.OverrideLevel = InLevel;
			IFA = InLevel->GetWorld()->SpawnActor<AInstancedFoliageActor>(SpawnParams);
			InLevel->InstancedFoliageActor = IFA;
		}
	}

	return IFA;
}


int32 AInstancedFoliageActor::GetOverlappingSphereCount(const UFoliageType* FoliageType, const FSphere& Sphere) const
{
	if (const FFoliageInfo* Info = FindInfo(FoliageType))
	{
		return Info->GetOverlappingSphereCount(Sphere);
	}

	return 0;
}


int32 AInstancedFoliageActor::GetOverlappingBoxCount(const UFoliageType* FoliageType, const FBox& Box) const
{
	if (const FFoliageInfo* Info = FindInfo(FoliageType))
	{
		return Info->GetOverlappingBoxCount(Box);
	}

	return 0;
}


void AInstancedFoliageActor::GetOverlappingBoxTransforms(const UFoliageType* FoliageType, const FBox& Box, TArray<FTransform>& OutTransforms) const
{
	if (const FFoliageInfo* Info = FindInfo(FoliageType))
	{
		Info->GetOverlappingBoxTransforms(Box, OutTransforms);
	}
}

void AInstancedFoliageActor::GetOverlappingMeshCounts(const FSphere& Sphere, TMap<UStaticMesh*, int32>& OutCounts) const
{
	for (auto& Pair : FoliageInfos)
	{
		FFoliageInfo const* Info = &*Pair.Value;

		if (Info)
		{
			Info->GetOverlappingMeshCount(Sphere, OutCounts);
		}
	}
}

bool AInstancedFoliageActor::ForEachFoliageInfo(TFunctionRef<bool(UFoliageType* FoliageType, FFoliageInfo& FoliageInfo)> InOperation)
{
	for (auto& Pair : FoliageInfos)
	{
		if (!InOperation(Pair.Key, Pair.Value.Get()))
		{
			return false;
		}
	}

	return true;
}

TUniqueObj<FFoliageInfo>& AInstancedFoliageActor::AddFoliageInfo(UFoliageType* FoliageType)
{
	TUniqueObj<FFoliageInfo>& NewFoliageInfo = FoliageInfos.Add(FoliageType);
#if WITH_EDITORONLY_DATA
	NewFoliageInfo->IFA = this;
	NewFoliageInfo->FoliageTypeUpdateGuid = FoliageType->UpdateGuid;
#endif
	return NewFoliageInfo;
}

TUniqueObj<FFoliageInfo>& AInstancedFoliageActor::AddFoliageInfo(UFoliageType* FoliageType, TUniqueObj<FFoliageInfo>&& FoliageInfo)
{
	TUniqueObj<FFoliageInfo>& NewFoliageInfo = FoliageInfos.Add(FoliageType, MoveTemp(FoliageInfo));
#if WITH_EDITORONLY_DATA
	NewFoliageInfo->IFA = this;
	NewFoliageInfo->FoliageTypeUpdateGuid = FoliageType->UpdateGuid;
#endif
	return NewFoliageInfo;
}

bool AInstancedFoliageActor::RemoveFoliageInfoAndCopyValue(UFoliageType* FoliageType, TUniqueObj<FFoliageInfo>& OutFoliageInfo)
{
	return FoliageInfos.RemoveAndCopyValue(FoliageType, OutFoliageInfo);
}

UFoliageType* AInstancedFoliageActor::GetLocalFoliageTypeForSource(const UObject* InSource, FFoliageInfo** OutMeshInfo)
{
	UFoliageType* ReturnType = nullptr;
	FFoliageInfo* Info = nullptr;

	for (auto& Pair : FoliageInfos)
	{
		UFoliageType* FoliageType = Pair.Key;
		// Check that the type is neither an asset nor blueprint instance

		if (FoliageType && FoliageType->GetSource() == InSource && !FoliageType->IsAsset() && FoliageType->GetClass()->IsNative())
		{
			ReturnType = FoliageType;
			Info = &*Pair.Value;
			break;
		}
	}

	if (OutMeshInfo)
	{
		*OutMeshInfo = Info;
	}

	return ReturnType;
}

void AInstancedFoliageActor::GetAllFoliageTypesForSource(const UObject* InSource, TArray<const UFoliageType*>& OutFoliageTypes)
{
	for (auto& Pair : FoliageInfos)
	{
		UFoliageType* FoliageType = Pair.Key;
		if (FoliageType && FoliageType->GetSource() == InSource)
		{
			OutFoliageTypes.Add(FoliageType);
		}
	}
}


FFoliageInfo* AInstancedFoliageActor::FindFoliageTypeOfClass(TSubclassOf<UFoliageType_InstancedStaticMesh> Class)
{
	FFoliageInfo* Info = nullptr;

	for (auto& Pair : FoliageInfos)
	{
		UFoliageType* FoliageType = Pair.Key;
		if (FoliageType && FoliageType->GetClass() == Class)
		{
			Info = &Pair.Value.Get();
			break;
		}
	}

	return Info;
}

FFoliageInfo* AInstancedFoliageActor::FindInfo(const UFoliageType* InType)
{
	TUniqueObj<FFoliageInfo>* InfoEntry = FoliageInfos.Find(InType);
	FFoliageInfo* Info = InfoEntry ? &InfoEntry->Get() : nullptr;
	return Info;
}

const FFoliageInfo* AInstancedFoliageActor::FindInfo(const UFoliageType* InType) const
{
	const TUniqueObj<FFoliageInfo>* InfoEntry = FoliageInfos.Find(InType);
	const FFoliageInfo* Info = InfoEntry ? &InfoEntry->Get() : nullptr;
	return Info;
}


#if WITH_EDITOR
AInstancedFoliageActor* AInstancedFoliageActor::Get(UWorld* InWorld, bool bCreateIfNone, ULevel* InLevelHint, const FVector& InLocationHint)
{
	UActorPartitionSubsystem* ActorPartitionSubsystem = InWorld->GetSubsystem<UActorPartitionSubsystem>();

	return Cast<AInstancedFoliageActor>(
		ActorPartitionSubsystem->GetActor(
			FActorPartitionGetParams(
				AInstancedFoliageActor::StaticClass(), 
				bCreateIfNone, 
				InLevelHint, 
				InLocationHint
			)
		)
	);
}


AInstancedFoliageActor* AInstancedFoliageActor::GetDefault(UWorld* InWorld)
{
	AInstancedFoliageActor* IFA = nullptr;
	ULevel* CurrentLevel = InWorld ? InWorld->GetCurrentLevel() : nullptr;
	if (CurrentLevel)
	{
		IFA = CurrentLevel->InstancedFoliageActor.Get();
		if (!IFA)
		{
			const bool bIsLevelPartition = CurrentLevel->GetWorld()->GetSubsystem<UActorPartitionSubsystem>()->IsLevelPartition();
			// In case Actor was already created for this level (this can't happen in other Partition modes)
			if (bIsLevelPartition)
			{
				for (AActor* ExistingActor : CurrentLevel->Actors)
				{
					IFA = Cast<AInstancedFoliageActor>(ExistingActor);
					if (IFA)
					{
						CurrentLevel->InstancedFoliageActor = IFA;
						return IFA;
					}
				}
			}

			FActorSpawnParameters SpawnParams;
			SpawnParams.ObjectFlags = RF_Transactional;
			if (!bIsLevelPartition)
			{
				SpawnParams.ObjectFlags |= RF_Transient;
			}
			SpawnParams.OverrideLevel = CurrentLevel;
			SpawnParams.bCreateActorPackage = true;
			IFA = InWorld->SpawnActor<AInstancedFoliageActor>(SpawnParams);
			CurrentLevel->InstancedFoliageActor = IFA;
		}
	}

	return IFA;
}

void AInstancedFoliageActor::MoveInstancesForMovedOwnedActors(AActor* InActor)
{
	// We don't want to handle this case when applying level transform
	// since it's already handled in AInstancedFoliageActor::PostApplyLevelTransform
	if (FLevelUtils::IsApplyingLevelTransform())
	{
		return;
	}
		
	for (auto& Pair : FoliageInfos)
	{
		// Source of movement is the Foliage
		if (Pair.Value->MovingInstances.Num() > 0)
		{
			continue;
		}

		if (Pair.Key->IsA<UFoliageType_Actor>() && Pair.Value->Type == EFoliageImplType::Actor)
		{
			if (FFoliageActor* FoliageActor = StaticCast<FFoliageActor*>(Pair.Value->Implementation.Get()))
			{
				// We might need to update the Owner IFA for this Actor.
				const int32 ActorIndex = FoliageActor->FindIndex(InActor);
				if (ActorIndex != INDEX_NONE)
				{
					AInstancedFoliageActor* TargetIFA = AInstancedFoliageActor::Get(GetWorld(), true, GetLevel(), InActor->GetActorLocation());
					FoliageActor->UpdateInstanceFromActor(ActorIndex, *Pair.Value);
					if (TargetIFA != this)
					{
						// After Moving the Actor doesn't have the same TargetIFA. Reassign.
						Pair.Value->MoveInstances(TargetIFA, { ActorIndex }, true);
					}
					break;
				}
			}
		}
	}
}

namespace FoliagePartitioningUtils
{
	static void Update(AInstancedFoliageActor* SourceIFA, FFoliageInfo& FoliageInfo, TFunctionRef<TSet<int32>* ()> GetInstanceSet)
	{
		UActorPartitionSubsystem* ActorPartitionSubsystem = SourceIFA->GetWorld()->GetSubsystem<UActorPartitionSubsystem>();
		if (FoliageInfo.Type == EFoliageImplType::Actor && ActorPartitionSubsystem->IsLevelPartition())
		{
			return; // Actors are handled through the Partitioning code
		}

		bool bMovedInstances = true;
		AInstancedFoliageActor* TargetIFA = nullptr;

		TSet<int32>* InstanceSet = GetInstanceSet();
		while (InstanceSet && InstanceSet->Num() > 0 && bMovedInstances)
		{
			TargetIFA = nullptr;
			bMovedInstances = false;
			TSet<int32> InstancesToMove;

			for (int32 InstanceIdx : *InstanceSet)
			{
				FFoliageInstance& Instance = FoliageInfo.Instances[InstanceIdx];
				AInstancedFoliageActor* NewIFA = AInstancedFoliageActor::Get(SourceIFA->GetWorld(), true, SourceIFA->GetLevel(), Instance.Location);
				if ((TargetIFA == nullptr || TargetIFA == NewIFA) && NewIFA != SourceIFA)
				{
					TargetIFA = NewIFA;
					InstancesToMove.Add(InstanceIdx);
				}
			}

			if (InstancesToMove.Num())
			{
				// TargetIFA can be null (if target is unloaded cell). Meaning instances will be deleted.
				FoliageInfo.MoveInstances(TargetIFA, InstancesToMove, true);
				bMovedInstances = true;
			}

			InstanceSet = GetInstanceSet();
		}
	}
}

void AInstancedFoliageActor::MoveSelectedInstancesToActorEditorContext(UWorld* InWorld)
{
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		IFA->ForEachFoliageInfo([IFA](UFoliageType* FoliageType, FFoliageInfo& FoliageInfo)
		{
			FoliagePartitioningUtils::Update(IFA, FoliageInfo, [&]() { return &FoliageInfo.SelectedIndices; });
			return true; // continue iteration
		});
	}
}

void AInstancedFoliageActor::UpdateInstancePartitioning(UWorld* InWorld)
{
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		FScopedActorEditorContextFromActor Context(IFA);
		IFA->ForEachFoliageInfo([IFA](UFoliageType* FoliageType, FFoliageInfo& FoliageInfo)
        {
			FoliagePartitioningUtils::Update(IFA, FoliageInfo, [&]() { return &FoliageInfo.SelectedIndices; });
			return true; // continue iteration
        });
	}
}

void AInstancedFoliageActor::UpdateInstancePartitioningForMovedComponent(UActorComponent* InComponent)
{
	const auto BaseId = InstanceBaseCache.GetInstanceBaseId(InComponent);
	if (BaseId == FFoliageInstanceBaseCache::InvalidBaseId)
	{
		return;
	}

	{
		FScopedActorEditorContextFromActor Context(this);
		for (auto& Pair : FoliageInfos)
		{
			FFoliageInfo& Info = *Pair.Value;
			FoliagePartitioningUtils::Update(this, Info, [&]() { return Info.ComponentHash.Find(BaseId); });
		}
	}
}

bool AInstancedFoliageActor::MoveInstancesForMovedComponent(UActorComponent* InComponent)
{
	// We don't want to handle this case when applying level transform
	// since it's already handled in AInstancedFoliageActor::PostApplyLevelTransform
	if (FLevelUtils::IsApplyingLevelTransform())
	{
		return false;
	}
		
	const auto BaseId = InstanceBaseCache.GetInstanceBaseId(InComponent);
	if (BaseId == FFoliageInstanceBaseCache::InvalidBaseId)
	{
		return false;
	}

	const auto CurrentBaseInfo = InstanceBaseCache.GetInstanceBaseInfo(BaseId);

	// Found an invalid base so don't try to move instances
	if (!CurrentBaseInfo.BasePtr.IsValid())
	{
		return false;
	}

	bool bFirst = true;
	const auto NewBaseInfo = InstanceBaseCache.UpdateInstanceBaseInfoTransform(InComponent);

	FMatrix DeltaTransform =
		FTranslationMatrix(-CurrentBaseInfo.CachedLocation) *
		FInverseRotationMatrix(CurrentBaseInfo.CachedRotation) *
		FScaleMatrix(NewBaseInfo.CachedDrawScale / CurrentBaseInfo.CachedDrawScale) *
		FRotationMatrix(NewBaseInfo.CachedRotation) *
		FTranslationMatrix(NewBaseInfo.CachedLocation);

	for (auto& Pair : FoliageInfos)
	{
		FFoliageInfo& Info = *Pair.Value;
		const auto* InstanceSet = Info.ComponentHash.Find(BaseId);
		if (InstanceSet && InstanceSet->Num())
		{
			if (bFirst)
			{
				bFirst = false;
				Modify();
			}

			Info.Implementation->BeginUpdate();

			for (int32 InstanceIndex : *InstanceSet)
			{
				FFoliageInstance& Instance = Info.Instances[InstanceIndex];

				Info.InstanceHash->RemoveInstance(Instance.Location, InstanceIndex);

				// Apply change
				FMatrix NewTransform =
					FRotationMatrix(Instance.Rotation) *
					FTranslationMatrix(Instance.Location) *
					DeltaTransform;

				// Extract rotation and position
				Instance.Location = NewTransform.GetOrigin();
				Instance.Rotation = NewTransform.Rotator();

				// Apply render data
				Info.Implementation->SetInstanceWorldTransform(InstanceIndex, Instance.GetInstanceWorldTransform(), true);

				// Re-add the new instance location to the hash
				Info.InstanceHash->InsertInstance(Instance.Location, InstanceIndex);
			}

			Info.Implementation->EndUpdate();
			Info.Refresh(true, false);
		}
	}

	return true;
}

void AInstancedFoliageActor::DeleteInstancesForComponent(UActorComponent* InComponent)
{
	const auto BaseId = InstanceBaseCache.GetInstanceBaseId(InComponent);
	// Instances with empty base has BaseId==InvalidBaseId, we should not delete these
	if (BaseId == FFoliageInstanceBaseCache::InvalidBaseId)
	{
		return;
	}

	for (auto& Pair : FoliageInfos)
	{
		FFoliageInfo& Info = *Pair.Value;
		const auto* InstanceSet = Info.ComponentHash.Find(BaseId);
		if (InstanceSet)
		{
			Info.RemoveInstances(InstanceSet->Array(), true);
		}
	}
}

void AInstancedFoliageActor::DeleteInstancesForComponent(UActorComponent* InComponent, const UFoliageType* FoliageType)
{
	const auto BaseId = InstanceBaseCache.GetInstanceBaseId(InComponent);
	// Instances with empty base has BaseId==InvalidBaseId, we should not delete these
	if (BaseId == FFoliageInstanceBaseCache::InvalidBaseId)
	{
		return;
	}

	FFoliageInfo* Info = FindInfo(FoliageType);
	if (Info)
	{
		const auto* InstanceSet = Info->ComponentHash.Find(BaseId);
		if (InstanceSet)
		{
			Info->RemoveInstances(InstanceSet->Array(), true);
		}
	}
}

void AInstancedFoliageActor::DeleteInstancesForComponent(UWorld* InWorld, UActorComponent* InComponent)
{
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = (*It);
		IFA->Modify();
		IFA->DeleteInstancesForComponent(InComponent);
	}
}

bool AInstancedFoliageActor::DeleteInstancesForProceduralFoliageComponentInternal(const FGuid& InProceduralGuid, bool bInRebuildTree, bool bInDeleteAll)
{
	bool bRemovedInstances = false;
	BeginUpdate();
	for (auto& Pair : FoliageInfos)
	{
		FFoliageInfo& Info = *Pair.Value;
		TArray<int32> InstancesToRemove;
		for (int32 InstanceIdx = 0; InstanceIdx < Info.Instances.Num(); InstanceIdx++)
		{
			if ((Info.Instances[InstanceIdx].ProceduralGuid == InProceduralGuid) || (bInDeleteAll && Info.Instances[InstanceIdx].ProceduralGuid.IsValid()))
			{
				InstancesToRemove.Add(InstanceIdx);
			}
		}

		if (InstancesToRemove.Num())
		{
			Info.RemoveInstances(InstancesToRemove, bInRebuildTree);
			bRemovedInstances = true;
		}
	}
	EndUpdate();
	// Clean up dead cross-level references
	FFoliageInstanceBaseCache::CompactInstanceBaseCache(this);

	return bRemovedInstances;
}

bool AInstancedFoliageActor::DeleteInstancesForAllProceduralFoliageComponents(bool bInRebuildTree)
{
	return DeleteInstancesForProceduralFoliageComponentInternal(FGuid(), bInRebuildTree, true);
}

bool AInstancedFoliageActor::DeleteInstancesForProceduralFoliageComponent(const UProceduralFoliageComponent* InProceduralFoliageComponent, bool bInRebuildTree)
{
	return DeleteInstancesForProceduralFoliageComponent(InProceduralFoliageComponent->GetProceduralGuid(), bInRebuildTree);
}

bool AInstancedFoliageActor::DeleteInstancesForProceduralFoliageComponent(const FGuid& InProceduralGuid, bool bInRebuildTree)
{
	return DeleteInstancesForProceduralFoliageComponentInternal(InProceduralGuid, bInRebuildTree, false);
}

bool AInstancedFoliageActor::ContainsInstancesFromProceduralFoliageComponent(const UProceduralFoliageComponent* InProceduralFoliageComponent)
{
	return ContainsInstancesFromProceduralFoliageComponent(InProceduralFoliageComponent->GetProceduralGuid());
}

bool AInstancedFoliageActor::ContainsInstancesFromProceduralFoliageComponent(const FGuid& InProceduralGuid)
{
	for (auto& Pair : FoliageInfos)
	{
		FFoliageInfo& Info = *Pair.Value;
		TArray<int32> InstancesToRemove;
		for (int32 InstanceIdx = 0; InstanceIdx < Info.Instances.Num(); InstanceIdx++)
		{
			if (Info.Instances[InstanceIdx].ProceduralGuid == InProceduralGuid)
			{
				// The procedural component is responsible for an instance
				return true;
			}
		}
	}

	return false;
}

void AInstancedFoliageActor::MoveInstancesForComponentToCurrentLevel(UActorComponent* InComponent)
{
	UWorld* InWorld = InComponent->GetWorld();
	MoveInstancesForComponentToLevel(InComponent, InWorld->GetCurrentLevel());
}

void AInstancedFoliageActor::MoveInstancesForComponentToLevel(UActorComponent* InComponent, ULevel* TargetLevel)
{
	if (!HasFoliageAttached(InComponent))
	{
		// Quit early if there are no foliage instances painted on this component
		return;
	}

	
	AInstancedFoliageActor* NewIFA = AInstancedFoliageActor::GetInstancedFoliageActorForLevel(TargetLevel, true);
	NewIFA->Modify();

	for (TActorIterator<AInstancedFoliageActor> It(InComponent->GetWorld()); It; ++It)
	{
		AInstancedFoliageActor* IFA = (*It);

		const auto SourceBaseId = IFA->InstanceBaseCache.GetInstanceBaseId(InComponent);
		if (SourceBaseId != FFoliageInstanceBaseCache::InvalidBaseId && IFA != NewIFA)
		{
			IFA->Modify();

			for (auto& Pair : IFA->FoliageInfos)
			{
				FFoliageInfo& Info = *Pair.Value;
				UFoliageType* FoliageType = Pair.Key;

				const auto* InstanceSet = Info.ComponentHash.Find(SourceBaseId);
				if (InstanceSet)
				{
					// Duplicate the foliage type if it's not shared
					FFoliageInfo* TargetMeshInfo = nullptr;
					UFoliageType* TargetFoliageType = NewIFA->AddFoliageType(FoliageType, &TargetMeshInfo);

					// Add the foliage to the new level
					for (int32 InstanceIndex : *InstanceSet)
					{
						TargetMeshInfo->AddInstance(TargetFoliageType, Info.Instances[InstanceIndex], InComponent);
					}

					TargetMeshInfo->Refresh(true, true);

					// Remove from old level
					Info.RemoveInstances(InstanceSet->Array(), true);
				}
			}
		}
	}
}

void AInstancedFoliageActor::MoveInstancesToNewComponent(UPrimitiveComponent* InOldComponent, const FBox& InBoxWithInstancesToMove, UPrimitiveComponent* InNewComponent)
{
	MoveInstancesToNewComponent(InOldComponent, InNewComponent, [InBoxWithInstancesToMove](const FFoliageInfo& FoliageInfo)
	{
		return FoliageInfo.GetInstancesOverlappingBox(InBoxWithInstancesToMove);
	});
}

void AInstancedFoliageActor::MoveInstancesToNewComponent(UPrimitiveComponent* InOldComponent, UPrimitiveComponent* InNewComponent)
{
	MoveInstancesToNewComponent(InOldComponent, InNewComponent, [](const FFoliageInfo& FoliageInfo)
	{
		TArray<int32> InstanceIndices;
		InstanceIndices.SetNum(FoliageInfo.Instances.Num());
		for (int32 InstanceIndex = 0; InstanceIndex < FoliageInfo.Instances.Num(); ++InstanceIndex)
		{
			InstanceIndices[InstanceIndex] = InstanceIndex;
		}
		return MoveTemp(InstanceIndices);
	});
}

void AInstancedFoliageActor::MoveInstancesToNewComponent(UPrimitiveComponent* InOldComponent, UPrimitiveComponent* InNewComponent, TFunctionRef<TArray<int32>(const FFoliageInfo&)> GetInstancesToMoveFunc)
{	
	const auto OldBaseId = InstanceBaseCache.GetInstanceBaseId(InOldComponent);
	if (OldBaseId == FFoliageInstanceBaseCache::InvalidBaseId)
	{
		// This foliage actor has no instances with specified base
		return;
	}
		
	ULevel* IFALevel = InNewComponent->GetOwner()->GetLevel();
	UWorld* IFAWorld = IFALevel->GetWorld();
	
	TArray<int32> InstancesToDelete;
	TSet<int32> InstancesToUpdateBase;
	TMap<AInstancedFoliageActor*, TArray<FFoliageInstance*>> PerIFAInstancesToMove;

	// If Modify was called on this IFA
	bool bModified = false;
	FScopedActorEditorContextFromActor Context(this);
	
	for (auto& Pair : FoliageInfos)
	{			
		FFoliageInfo& Info = *Pair.Value;
		// Make sure we indeed have Instances that have the OldComponent as a base for this specific FoliageType
		TSet<int32>* OldInstanceSet = Info.ComponentHash.Find(OldBaseId);
		if(OldInstanceSet && OldInstanceSet->Num())
		{
			FFoliageInstanceBaseId NewBaseId = FFoliageInstanceBaseCache::InvalidBaseId;
			TArray<int32> PotentialInstances = GetInstancesToMoveFunc(Info);

			// Reset temp containers
			PerIFAInstancesToMove.Reset();
			InstancesToUpdateBase.Reset();
			InstancesToDelete.Reset(PotentialInstances.Num());

			// Cumulate Instances to move Per IFA
			for (int32 InstanceIndex : PotentialInstances)
			{
				if (Info.Instances.IsValidIndex(InstanceIndex) && OldInstanceSet->Contains(InstanceIndex))
				{
					FFoliageInstance& InstanceToMove = Info.Instances[InstanceIndex];
					const bool bCreate = true;
					if (AInstancedFoliageActor* TargetIFA = AInstancedFoliageActor::Get(IFAWorld, bCreate, IFALevel, InstanceToMove.Location))
					{
						// Call Modify only once
						if (!bModified)
						{
							Modify();
							bModified = true;
						}

						// Same IFA just update the Base 
						if (this == TargetIFA)
						{
							if (NewBaseId == FFoliageInstanceBaseCache::InvalidBaseId)
							{
								NewBaseId = InstanceBaseCache.AddInstanceBaseId(InNewComponent);
							}
							InstanceToMove.BaseComponent = InNewComponent;
							InstanceToMove.BaseId = NewBaseId;
							InstancesToUpdateBase.Add(InstanceIndex);
							OldInstanceSet->Remove(InstanceIndex);
						}
						else
						{
							TArray<FFoliageInstance*>& InstancesToMove = PerIFAInstancesToMove.FindOrAdd(TargetIFA);
							InstancesToMove.Add(&InstanceToMove);
							InstancesToDelete.Add(InstanceIndex);
						}
					}
				}
			}

			// Add Instances to IFAs
			for (auto& IFAPair : PerIFAInstancesToMove)
			{
				AInstancedFoliageActor* TargetIFA = IFAPair.Key;
				TargetIFA->Modify();
				FFoliageInfo* TargetMeshInfo = nullptr;
				UFoliageType* TargetFoliageType = TargetIFA->AddFoliageType(Pair.Key, &TargetMeshInfo);
				for (FFoliageInstance* InstanceToMove : IFAPair.Value)
				{
					TargetMeshInfo->AddInstance(TargetFoliageType, *InstanceToMove, InNewComponent);
				}
				TargetMeshInfo->Refresh(true, true);
			}
									
			// Remove old set if empty
			if (!OldInstanceSet->Num())
			{
				Info.ComponentHash.Remove(OldBaseId);
			}
			// Add instances that are still in the old ifa but with a new base
			if (InstancesToUpdateBase.Num())
			{
				check(NewBaseId != FFoliageInstanceBaseCache::InvalidBaseId);
				Info.ComponentHash.Add(NewBaseId, InstancesToUpdateBase);
			}
			// Remove from old IFA
			if (InstancesToDelete.Num())
			{
				Info.RemoveInstances(InstancesToDelete, true);
			}
		}
	}
}

void AInstancedFoliageActor::MoveInstancesToNewComponent(UWorld* InWorld, UPrimitiveComponent* InOldComponent, UPrimitiveComponent* InNewComponent)
{
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = (*It);
		IFA->MoveInstancesToNewComponent(InOldComponent, InNewComponent);
	}
}

void AInstancedFoliageActor::MoveInstancesToNewComponent(UWorld* InWorld, UPrimitiveComponent* InOldComponent, const FBox& InBoxWithInstancesToMove, UPrimitiveComponent* InNewComponent)
{
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = (*It);
		IFA->MoveInstancesToNewComponent(InOldComponent, InBoxWithInstancesToMove, InNewComponent);
	}
}

void AInstancedFoliageActor::MoveInstancesToLevel(ULevel* InTargetLevel, TSet<int32>& InInstanceList, FFoliageInfo* InCurrentMeshInfo, UFoliageType* InFoliageType, bool bSelect)
{
	if (InTargetLevel == GetLevel())
	{
		return;
	}

	AInstancedFoliageActor* TargetIFA = GetInstancedFoliageActorForLevel(InTargetLevel, /*bCreateIfNone*/ true);
	InCurrentMeshInfo->MoveInstances(TargetIFA, InInstanceList, bSelect);
}

void AInstancedFoliageActor::MoveSelectedInstancesToLevel(ULevel* InTargetLevel)
{
	if (InTargetLevel == GetLevel() || !HasSelectedInstances())
	{
		return;
	}

	for (auto& Pair : FoliageInfos)
	{
		FFoliageInfo& Info = *Pair.Value;
		UFoliageType* FoliageType = Pair.Key;

		MoveInstancesToLevel(InTargetLevel, Info.SelectedIndices, &Info, FoliageType);
	}
}

void AInstancedFoliageActor::MoveAllInstancesToLevel(ULevel* InTargetLevel)
{
	if (InTargetLevel == GetLevel())
	{
		return;
	}

	for (auto& Pair : FoliageInfos)
	{
		FFoliageInfo& Info = *Pair.Value;
		UFoliageType* FoliageType = Pair.Key;

		TSet<int32> instancesList;

		for (int32 i = 0; i < Info.Instances.Num(); ++i)
		{
			instancesList.Add(i);
		}

		MoveInstancesToLevel(InTargetLevel, instancesList, &Info, FoliageType);
	}
}

TMap<UFoliageType*, TArray<const FFoliageInstancePlacementInfo*>> AInstancedFoliageActor::GetInstancesForComponent(UActorComponent* InComponent)
{
	TMap<UFoliageType*, TArray<const FFoliageInstancePlacementInfo*>> Result;
	const auto BaseId = InstanceBaseCache.GetInstanceBaseId(InComponent);

	if (BaseId != FFoliageInstanceBaseCache::InvalidBaseId)
	{
		for (auto& Pair : FoliageInfos)
		{
			const FFoliageInfo& Info = *Pair.Value;
			const auto* InstanceSet = Info.ComponentHash.Find(BaseId);
			if (InstanceSet)
			{
				TArray<const FFoliageInstancePlacementInfo*>& Array = Result.Add(Pair.Key, TArray<const FFoliageInstancePlacementInfo*>());
				Array.Empty(InstanceSet->Num());

				for (int32 InstanceIndex : *InstanceSet)
				{
					const FFoliageInstancePlacementInfo* Instance = &Info.Instances[InstanceIndex];
					Array.Add(Instance);
				}
			}
		}
	}

	return Result;
}

FFoliageInfo* AInstancedFoliageActor::FindOrAddMesh(UFoliageType* InType)
{
	TUniqueObj<FFoliageInfo>* MeshInfoEntry = FoliageInfos.Find(InType);
	FFoliageInfo* Info = MeshInfoEntry ? &MeshInfoEntry->Get() : AddMesh(InType);
	return Info;
}

UFoliageType* AInstancedFoliageActor::AddFoliageType(const UFoliageType* InType, FFoliageInfo** OutInfo)
{
	FFoliageInfo* Info = nullptr;
	UFoliageType* FoliageType = const_cast<UFoliageType*>(InType);

	if (FoliageType->GetOuter() == this || FoliageType->IsAsset())
	{
		auto ExistingMeshInfo = FoliageInfos.Find(FoliageType);
		if (!ExistingMeshInfo)
		{
			Modify();
			Info = &AddFoliageInfo(FoliageType).Get();
		}
		else
		{
			Info = &ExistingMeshInfo->Get();
		}
	}
	else if (FoliageType->GetClass()->ClassGeneratedBy)
	{
		// Foliage type blueprint
		FFoliageInfo* ExistingMeshInfo = FindFoliageTypeOfClass(FoliageType->GetClass());
		if (!ExistingMeshInfo)
		{
			Modify();
			FoliageType = DuplicateObject<UFoliageType>(InType, this);
			Info = &AddFoliageInfo(FoliageType).Get();
		}
		else
		{
			Info = ExistingMeshInfo;
		}
	}
	else
	{
		// Unique meshes only
		// Multiple entries for same static mesh can be added using FoliageType as an asset
		FoliageType = GetLocalFoliageTypeForSource(FoliageType->GetSource(), &Info);
		if (FoliageType == nullptr)
		{
			Modify();
			FoliageType = DuplicateObject<UFoliageType>(InType, this);
			Info = &AddFoliageInfo(FoliageType).Get();
		}
	}

	if (Info &&!Info->Implementation.IsValid())
	{
		Info->CreateImplementation(FoliageType);
		check(Info->Implementation.IsValid());
	}

	if (OutInfo)
	{
		*OutInfo = Info;
	}

	
	return FoliageType;
}

FFoliageInfo* AInstancedFoliageActor::AddMesh(UStaticMesh* InMesh, UFoliageType** OutSettings, const UFoliageType_InstancedStaticMesh* DefaultSettings)
{
	// This function is deprecated in a partitioned world.
	// FoliageType cannot have an AInstancedFoliageActor as their Outer
	// This creates issues with the Foliage Edit mode.
	// Proper way is to Create an asset for the UFoliageType and call
	// the InstancedFoliageActor::AddMesh(UFoliageType*) overload with the asset
	check(!GetWorld()->IsPartitionedWorld());

	check(GetLocalFoliageTypeForSource(InMesh) == nullptr);

	MarkPackageDirty();

	UFoliageType_InstancedStaticMesh* Settings = nullptr;
#if WITH_EDITORONLY_DATA
	if (DefaultSettings)
	{
		// TODO: Can't we just use this directly?
		FObjectDuplicationParameters DuplicationParameters(const_cast<UFoliageType_InstancedStaticMesh*>(DefaultSettings), this);
		DuplicationParameters.ApplyFlags = RF_Transactional;
		Settings = CastChecked<UFoliageType_InstancedStaticMesh>(StaticDuplicateObjectEx(DuplicationParameters));
	}
	else
#endif
	{
		Settings = NewObject<UFoliageType_InstancedStaticMesh>(this, NAME_None, RF_Transactional);
	}
	Settings->SetStaticMesh(InMesh);
	FFoliageInfo* Info = AddMesh(Settings);
	
	if (OutSettings)
	{
		*OutSettings = Settings;
	}

	return Info;
}

FFoliageInfo* AInstancedFoliageActor::AddMesh(UFoliageType* InType)
{
	check(!GetWorld()->IsPartitionedWorld() || InType->IsAsset());

	check(FoliageInfos.Find(InType) == nullptr);

	Modify();

	FFoliageInfo* Info = &*AddFoliageInfo(InType);
	if (!Info->Implementation.IsValid())
	{
		Info->CreateImplementation(InType);
	}
	Info->FoliageTypeUpdateGuid = InType->UpdateGuid;
	InType->IsSelected = true;

	return Info;
}

void AInstancedFoliageActor::RemoveFoliageType(UFoliageType** InFoliageTypes, int32 Num)
{
	Modify();
	UnregisterAllComponents();

	// Remove all components for this mesh from the Components array.
	for (int32 FoliageTypeIdx = 0; FoliageTypeIdx < Num; ++FoliageTypeIdx)
	{
		const UFoliageType* FoliageType = InFoliageTypes[FoliageTypeIdx];
		FFoliageInfo* Info = FindInfo(FoliageType);
		if (Info)
		{
			if (Info->IsInitialized())
			{
				Info->Uninitialize();
			}

			FoliageInfos.Remove(FoliageType);
		}
	}

	RegisterAllComponents();
}

void AInstancedFoliageActor::ClearSelection()
{
	for (TActorIterator<AInstancedFoliageActor> It(GetWorld()); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		for (auto& Pair : IFA->FoliageInfos)
		{
			FFoliageInfo& Info = *Pair.Value;
			Info.ClearSelection();
		}
	}
}

void AInstancedFoliageActor::SelectInstance(UInstancedStaticMeshComponent* InComponent, int32 InComponentInstanceIndex, bool bToggle)
{
	Modify(false);

	// If we're not toggling, we need to first deselect everything else
	if (!bToggle)
	{
		ClearSelection();
	}

	if (InComponent)
	{
		UFoliageType* Type = nullptr;
		FFoliageInfo* Info = nullptr;
		int32 InstanceIndex = INDEX_NONE;

		for (auto& Pair : FoliageInfos)
		{
			InstanceIndex = Pair.Value->Implementation->GetInstanceIndexFrom(InComponent, InComponentInstanceIndex);
			if (InstanceIndex != INDEX_NONE)
			{
				Type = Pair.Key;
				Info = &Pair.Value.Get();
				break;
			}
		}

		if (Info)
		{
			bool bIsSelected = Info->SelectedIndices.Contains(InstanceIndex);

			// Deselect if it's already selected.
			Info->Implementation->SelectInstance(false, InstanceIndex);

			if (bIsSelected)
			{
				Info->SelectedIndices.Remove(InstanceIndex);
			}

			if (!bToggle || !bIsSelected)
			{
				// Add the selection
				Info->Implementation->SelectInstance(true, InstanceIndex);

				Info->SelectedIndices.Add(InstanceIndex);
			}
		}
	}
}

bool AInstancedFoliageActor::SelectInstance(AActor* InActor, bool bToggle)
{
	if (InActor)
	{
		UFoliageType* Type = nullptr;
		FFoliageInfo* Info = nullptr;
		FFoliageActor* FoliageActor = nullptr;
		int32 index = INDEX_NONE;

		for (auto& Pair : FoliageInfos)
		{
			if (Pair.Value->Type == EFoliageImplType::Actor)
			{
				FFoliageActor* CurrentFoliageActor = StaticCast<FFoliageActor*>(Pair.Value->Implementation.Get());
				index = CurrentFoliageActor->FindIndex(InActor);
				if (index != INDEX_NONE)
				{
					Info = &Pair.Value.Get();
					FoliageActor = CurrentFoliageActor;
					break;
				}
			}
		}

		if (!Info)
		{
			return false;
		}

		Modify(false);

		// If we're not toggling, we need to first deselect everything else
		if (!bToggle)
		{
			ClearSelection();
		}

		bool bIsSelected = Info->SelectedIndices.Contains(index);

		FoliageActor->SelectInstance(false, index);

		if (bIsSelected)
		{
			Info->SelectedIndices.Remove(index);
		}

		if (!bToggle || !bIsSelected)
		{
			// Add the selection
			FoliageActor->SelectInstance(true, index);

			Info->SelectedIndices.Add(index);
		}
	}
	return true;
}

FOLIAGE_API FBox AInstancedFoliageActor::GetSelectionBoundingBox() const
{
	FBox SelectionBoundingBox(EForceInit::ForceInit);

	for (auto& Pair : FoliageInfos)
	{
		const FFoliageInfo* Info = &Pair.Value.Get();
		SelectionBoundingBox += Info->GetSelectionBoundingBox();
	}

	return SelectionBoundingBox;
}

bool AInstancedFoliageActor::HasSelectedInstances() const
{
	for (const auto& Pair : FoliageInfos)
	{
		if (Pair.Value->SelectedIndices.Num() > 0)
		{
			return true;
		}
	}

	return false;
}

const UFoliageType* AInstancedFoliageActor::FindFoliageType(const FFoliageInfo* InFoliageInfo) const
{
	for (auto& Pair : FoliageInfos)
	{
		if (&Pair.Value.Get() == InFoliageInfo)
		{
			return Pair.Key;
		}
	}

	return nullptr;
}

TMap<UFoliageType*, FFoliageInfo*> AInstancedFoliageActor::GetAllInstancesFoliageType()
{
	TMap<UFoliageType*, FFoliageInfo*> InstanceFoliageTypes;

	for (auto& Pair : FoliageInfos)
	{
		InstanceFoliageTypes.Add(Pair.Key, &Pair.Value.Get());
	}

	return InstanceFoliageTypes;
}

TMap<UFoliageType*, FFoliageInfo*> AInstancedFoliageActor::GetSelectedInstancesFoliageType()
{
	TMap<UFoliageType*, FFoliageInfo*> SelectedInstanceFoliageTypes;

	for (auto& Pair : FoliageInfos)
	{
		if (Pair.Value->SelectedIndices.Num() > 0)
		{
			SelectedInstanceFoliageTypes.Add(Pair.Key, &Pair.Value.Get());
		}
	}

	return SelectedInstanceFoliageTypes;
}

void AInstancedFoliageActor::Destroyed()
{
	if (GIsEditor && !GetWorld()->IsGameWorld())
	{
		for (auto& Pair : FoliageInfos)
		{
			if (Pair.Value->Type == EFoliageImplType::StaticMesh)
			{
				FFoliageStaticMesh* FoliageStaticMesh = StaticCast<FFoliageStaticMesh*>(Pair.Value->Implementation.Get());
				UHierarchicalInstancedStaticMeshComponent* Component = FoliageStaticMesh->Component;

				if (Component)
				{
					Component->ClearInstances();
					// Save the component's PendingKill flag to restore the component if the delete is undone.
					Component->SetFlags(RF_Transactional);
					Component->Modify();
				}
			}
			else if (Pair.Value->Type == EFoliageImplType::Actor)
			{
				FFoliageActor* FoliageActor = StaticCast<FFoliageActor*>(Pair.Value->Implementation.Get());
				FoliageActor->DestroyActors(false);
			}
		}
		FoliageInfos.Empty();
	}

	Super::Destroyed();
}

void AInstancedFoliageActor::PreEditUndo()
{
	Super::PreEditUndo();

	// Remove all delegate as we dont know what the Undo will affect and we will simply readd those still valid afterward
	for (auto& Pair : FoliageInfos)
	{
		FFoliageInfo& Info = *Pair.Value;

		Info.PreEditUndo(Pair.Key);
	}
}

void AInstancedFoliageActor::PostEditUndo()
{
	Super::PostEditUndo();

	FlushRenderingCommands();

	InstanceBaseCache.UpdateInstanceBaseCachedTransforms();

	BeginUpdate();

	for (auto& Pair : FoliageInfos)
	{
		FFoliageInfo& Info = *Pair.Value;

		Info.PostEditUndo(Pair.Key);
	}
	EndUpdate();
}

void AInstancedFoliageActor::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (!bDuplicateForPIE)
	{
		// Fix up in case some code path duplicates an IFA outside of PIE (Duplicate of a Map)
		for (auto& Pair : FoliageInfos)
		{
			FFoliageInfo& Info = *Pair.Value;
			Info.IFA = this;
			if (FFoliageImpl* Impl = Info.Implementation.Get())
			{
				Impl->Info = &Info;
			}
		}
	}
}

bool AInstancedFoliageActor::ShouldExport()
{
	// We don't support exporting/importing InstancedFoliageActor itself
	// Instead foliage instances exported/imported together with components it's painted on
	return false;
}

bool AInstancedFoliageActor::ShouldImport(FStringView ActorPropString, bool IsMovingLevel)
{
	return false;
}

void AInstancedFoliageActor::ApplySelection(bool bApply)
{
	for (auto& Pair : FoliageInfos)
	{
		FFoliageInfo& Info = *Pair.Value;
				
		Info.Implementation->ApplySelection(bApply, Info.SelectedIndices);
	}
}

bool AInstancedFoliageActor::GetSelectionLocation(FBox& OutBox) const
{
	// Could probably be cached instead of recalculated always.
	bool bHasSelection = false;
	for (const auto& Pair : FoliageInfos)
	{
		const FFoliageInfo& Info = Pair.Value.Get();
		for(int32 InstanceIdx : Info.SelectedIndices)
		{
			OutBox += Info.Instances[InstanceIdx].Location;
			bHasSelection = true;
		}
	}
		
	return bHasSelection;
}

bool AInstancedFoliageActor::HasFoliageAttached(UActorComponent* InComponent)
{
	for (TActorIterator<AInstancedFoliageActor> It(InComponent->GetWorld()); It; ++It)
	{
		AInstancedFoliageActor* IFA = (*It);
		if (IFA->InstanceBaseCache.GetInstanceBaseId(InComponent) != FFoliageInstanceBaseCache::InvalidBaseId)
		{
			return true;
		}
	}

	return false;
}


void AInstancedFoliageActor::MapRebuild()
{
	// Map rebuild may have modified the BSP's ModelComponents and thrown the previous ones away.
	// Most BSP-painted foliage is attached to a Brush's UModelComponent which persist across rebuilds,
	// but any foliage attached directly to the level BSP's ModelComponents will need to try to find a new base.

	CleanupDeletedFoliageType();

	TMap<UFoliageType*, TArray<FFoliageInstance>> NewInstances;
	TArray<UModelComponent*> RemovedModelComponents;
	UWorld* World = GetWorld();
	check(World);

	// For each foliage brush, represented by the mesh/info pair
	for (auto& Pair : FoliageInfos)
	{
		// each target component has some foliage instances
		FFoliageInfo const& Info = *Pair.Value;
		UFoliageType* Settings = Pair.Key;
		check(Settings);

		for (auto& ComponentFoliagePair : Info.ComponentHash)
		{
			// BSP components are UModelComponents - they are the only ones we need to change
			auto BaseComponentPtr = InstanceBaseCache.GetInstanceBasePtr(ComponentFoliagePair.Key);
			UModelComponent* TargetComponent = Cast<UModelComponent>(BaseComponentPtr.Get());

			// Check if it's part of a brush. We only need to fix up model components that are part of the level BSP.
			if (TargetComponent && Cast<ABrush>(TargetComponent->GetOuter()) == nullptr)
			{
				// Delete its instances later
				RemovedModelComponents.Add(TargetComponent);

				// We have to test each instance to see if we can migrate it across
				for (int32 InstanceIdx : ComponentFoliagePair.Value)
				{
					// Use a line test against the world. This is not very reliable as we don't know the original trace direction.
					check(Info.Instances.IsValidIndex(InstanceIdx));
					FFoliageInstance const& Instance = Info.Instances[InstanceIdx];

					FFoliageInstance NewInstance = Instance;

					FTransform InstanceToWorld = Instance.GetInstanceWorldTransform();
					FVector Down(-FVector::UpVector);
					FVector Start(InstanceToWorld.TransformPosition(FVector::UpVector));
					FVector End(InstanceToWorld.TransformPosition(Down));

					FHitResult Result;
					bool bHit = World->LineTraceSingleByObjectType(Result, Start, End, FCollisionObjectQueryParams(ECC_WorldStatic), FCollisionQueryParams(NAME_None, FCollisionQueryParams::GetUnknownStatId(), true));

					if (bHit && Result.Component.IsValid() && Result.Component->IsA(UModelComponent::StaticClass()))
					{
						NewInstance.BaseId = InstanceBaseCache.AddInstanceBaseId(Result.Component.Get());
						NewInstances.FindOrAdd(Settings).Add(NewInstance);
					}
				}
			}
		}
	}

	// Remove all existing & broken instances & component references.
	for (UModelComponent* Component : RemovedModelComponents)
	{
		DeleteInstancesForComponent(Component);
	}

	// And then finally add our new instances to the correct target components.
	for (auto& NewInstancePair : NewInstances)
	{
		UFoliageType* Settings = NewInstancePair.Key;
		check(Settings);
		FFoliageInfo& Info = *FindOrAddMesh(Settings);
		for (FFoliageInstance& Instance : NewInstancePair.Value)
		{
			Info.AddInstance(Settings, Instance);
		}

		Info.Refresh(true, true);
	}
}

#endif // WITH_EDITOR

struct FFoliageMeshInfo_Old
{
	TArray<FFoliageInstanceCluster_Deprecated> InstanceClusters;
	TArray<FFoliageInstance_Deprecated> Instances;
	UFoliageType_InstancedStaticMesh* Settings; // Type remapped via +ActiveClassRedirects
};
FArchive& operator<<(FArchive& Ar, FFoliageMeshInfo_Old& MeshInfo)
{
	Ar << MeshInfo.InstanceClusters;
	Ar << MeshInfo.Instances;
	Ar << MeshInfo.Settings;

	return Ar;
}

void AInstancedFoliageActor::Serialize(FArchive& Ar)
{
	LLM_SCOPE_BYNAME(TEXT("Foliage"));
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFoliageCustomVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (!Ar.ArIsFilterEditorOnly && Ar.CustomVer(FFoliageCustomVersion::GUID) >= FFoliageCustomVersion::CrossLevelBase)
	{
		Ar << InstanceBaseCache;
	}
#endif

	if (Ar.UEVer() < VER_UE4_FOLIAGE_SETTINGS_TYPE)
	{
#if WITH_EDITORONLY_DATA
		TMap<UFoliageType*, TUniqueObj<FFoliageMeshInfo_Deprecated>> FoliageMeshesDeprecated;
		TMap<UStaticMesh*, FFoliageMeshInfo_Old> OldFoliageMeshes;
		Ar << OldFoliageMeshes;
		for (auto& OldMeshInfo : OldFoliageMeshes)
		{
			FFoliageMeshInfo_Deprecated NewMeshInfo;

			NewMeshInfo.Instances = MoveTemp(OldMeshInfo.Value.Instances);

			UFoliageType_InstancedStaticMesh* FoliageType = OldMeshInfo.Value.Settings;
			if (FoliageType == nullptr)
			{
				// If the Settings object was null, eg the user forgot to save their settings asset, create a new one.
				FoliageType = NewObject<UFoliageType_InstancedStaticMesh>(this);
			}

			if (FoliageType->Mesh == nullptr)
			{
				FoliageType->Modify();
				FoliageType->Mesh = OldMeshInfo.Key;
			}
			else if (FoliageType->Mesh != OldMeshInfo.Key)
			{
				// If mesh doesn't match (two meshes sharing the same settings object?) then we need to duplicate as that is no longer supported
				FoliageType = (UFoliageType_InstancedStaticMesh*)StaticDuplicateObject(FoliageType, this, NAME_None, RF_AllFlags & ~(RF_Standalone | RF_Public));
				FoliageType->Mesh = OldMeshInfo.Key;
			}
			NewMeshInfo.FoliageTypeUpdateGuid = FoliageType->UpdateGuid;
			FoliageMeshes_Deprecated.Add(FoliageType, TUniqueObj<FFoliageMeshInfo_Deprecated>(MoveTemp(NewMeshInfo)));
		}
#endif//WITH_EDITORONLY_DATA
	}
	else
	{
		if (Ar.CustomVer(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::CrossLevelBase)
		{
#if WITH_EDITORONLY_DATA
			Ar << FoliageMeshes_Deprecated;
#endif
		}
		else if(Ar.CustomVer(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::FoliageActorSupport)
		{
#if WITH_EDITORONLY_DATA
			Ar << FoliageMeshes_Deprecated2;
#endif
		}
		else
		{
			Ar << FoliageInfos;
		}
	}

	// Clean up any old cluster components and convert to hierarchical instanced foliage.
	if (Ar.CustomVer(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::FoliageUsingHierarchicalISMC)
	{
		for (UActorComponent* Component : GetComponents())
		{
			if (Cast<UInstancedStaticMeshComponent>(Component))
			{
				Component->bAutoRegister = false;
			}
		}
	}
	
	for (auto& Pair : FoliageInfos)
	{
		FFoliageInfo& Info = *Pair.Value;
#if WITH_EDITORONLY_DATA
		if (Ar.IsLoading())
		{
			// Fix-up the IFA pointer as it's not serialized
			Info.IFA = this;
		}
#endif
		Info.PostSerialize(Ar);
	}
}

#if WITH_EDITOR
UActorComponent* AInstancedFoliageActor::GetBaseComponentFromBaseId(const FFoliageInstanceBaseId& BaseId) const
{
	return InstanceBaseCache.GetInstanceBasePtr(BaseId).Get();
}

AInstancedFoliageActor::FOnSelectionChanged AInstancedFoliageActor::SelectionChanged;
AInstancedFoliageActor::FOnInstanceCoundChanged AInstancedFoliageActor::InstanceCountChanged;

void AInstancedFoliageActor::EnterEditMode()
{
	for (auto& FoliageMesh : FoliageInfos)
	{
		FoliageMesh.Value->EnterEditMode();
	}
}

void AInstancedFoliageActor::ExitEditMode()
{
	for (auto& FoliageMesh : FoliageInfos)
	{
		FoliageMesh.Value->ExitEditMode();
	}
}

void AInstancedFoliageActor::PostInitProperties()
{
	Super::PostInitProperties();

	if (!IsTemplate())
	{
		if (HasAnyFlags(RF_NeedPostLoad) || GetOuter()->HasAnyFlags(RF_NeedPostLoad))
		{
			// Delegate registration is not thread-safe, so we postpone it on PostLoad when coming from loading which could be on another thread
		}
	}
}

void AInstancedFoliageActor::BeginDestroy()
{
	Super::BeginDestroy();

	if (!IsTemplate())
	{
		FoliageInfos.Empty();
	}
}

bool AInstancedFoliageActor::IsListedInSceneOutliner() const
{
	// In World Partition the AInstancedFoliageActor::GetDefault creates a transient foliage actor to store unpainted foliage types.
	return !HasAnyFlags(RF_Transient);
}

#endif

void AInstancedFoliageActor::PostLoad()
{
	Super::PostLoad();

	ULevel* OwningLevel = GetLevel();
	// We can't check the ActorPartitionSubsystem here because World is not initialized yet. So we fallback on the bIsPartitioned
	// to know if multiple InstanceFoliageActors is valid or not. 
	// For levels that are World Partition runtime cells. Having multiple IFAs is valid so skip validation.
	if (OwningLevel && !OwningLevel->bIsPartitioned && !OwningLevel->IsWorldPartitionRuntimeCell() && !APartitionActor::IsPartitioned(this))
	{
		if (!OwningLevel->InstancedFoliageActor.IsValid())
		{
			OwningLevel->InstancedFoliageActor = this;
		}
		else
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("Level"), FText::FromString(*OwningLevel->GetOutermost()->GetName()));
			FMessageLog("MapCheck").Warning()
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_DuplicateInstancedFoliageActor", "Level {Level} has an unexpected duplicate Instanced Foliage Actor."), Arguments)))
#if WITH_EDITOR
				->AddToken(FActionToken::Create(LOCTEXT("MapCheck_FixDuplicateInstancedFoliageActor", "Fix"),
					LOCTEXT("MapCheck_FixDuplicateInstancedFoliageActor_Desc", "Click to consolidate foliage into the main foliage actor."),
					FOnActionTokenExecuted::CreateUObject(OwningLevel->InstancedFoliageActor.Get(), &AInstancedFoliageActor::RepairDuplicateIFA, this), true))
#endif// WITH_EDITOR
				;
			FMessageLog("MapCheck").Open(EMessageSeverity::Warning);
		}
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		if (GetLinkerCustomVersion(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::CrossLevelBase)
		{
			ConvertDeprecatedFoliageMeshes(this, FoliageMeshes_Deprecated, ObjectPtrDecay(FoliageInfos));
			FoliageMeshes_Deprecated.Empty();
		}
		else if (GetLinkerCustomVersion(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::FoliageActorSupport)
		{
			ConvertDeprecated2FoliageMeshes(this, FoliageMeshes_Deprecated2, ObjectPtrDecay(FoliageInfos));
			FoliageMeshes_Deprecated2.Empty();
		}
			   
		{
			if (FoliageInfos.Contains(nullptr))
			{
				FMessageLog("MapCheck").Warning()
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_FoliageMissingStaticMesh", "Foliage instances for a missing static mesh have been removed.")))
					->AddToken(FMapErrorToken::Create(FMapErrors::FoliageMissingStaticMesh));
								
				TUniqueObj<FFoliageInfo> FoliageInfo;
				while (FoliageInfos.RemoveAndCopyValue(nullptr, FoliageInfo))
				{
					if (FoliageInfo->IsInitialized())
					{
						FoliageInfo->Uninitialize();
					}
				}				
			}
		}

		TArray<UFoliageType*> FoliageTypeToRemove;

		for (auto& Pair : FoliageInfos)
		{
			// Find the per-mesh info matching the mesh.
			FFoliageInfo& Info = *Pair.Value;
			// Make sure to set that before doing anything else (might already been done in Serialize except if the post load as upgraded/modified data)
			Info.IFA = this;
			
			UFoliageType* FoliageType = Pair.Key;

			// Make sure the source data has been PostLoaded as if not it can be considered invalid resulting in a bad HISMC tree
			UObject* Source = FoliageType->GetSource();
			if (Source)
			{
				Source->ConditionalPostLoad();
			}

			if (Info.Instances.Num() && !Info.IsInitialized())
			{
				FFormatNamedArguments Arguments;
				if (Source)
				{
					Arguments.Add(TEXT("MeshName"), FText::FromString(Source->GetName()));
				}
				else
				{
					Arguments.Add(TEXT("MeshName"), FText::FromString(TEXT("None")));
				}

				FMessageLog("MapCheck").Warning()
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_FoliageMissingComponent", "Foliage in this map is missing a component for static mesh {MeshName}. This has been repaired."), Arguments)))
					->AddToken(FMapErrorToken::Create(FMapErrors::FoliageMissingClusterComponent));

				Info.ReallocateClusters(Pair.Key);
			}
			
			Info.RecomputeHash();

			// Convert to Hierarchical foliage
			if (GetLinkerCustomVersion(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::FoliageUsingHierarchicalISMC)
			{
				Info.ReallocateClusters(Pair.Key);
			}

			if (GetLinkerCustomVersion(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::HierarchicalISMCNonTransactional)
			{
				if (ensure(Info.Type == EFoliageImplType::StaticMesh))
				{
					FFoliageStaticMesh* FoliageStaticMesh = StaticCast<FFoliageStaticMesh*>(Info.Implementation.Get());
					if (FoliageStaticMesh->Component)
					{
						FoliageStaticMesh->Component->ClearFlags(RF_Transactional);
					}
				}
			}

			// Clean up case where embeded instances had their static mesh deleted
			if (FoliageType->IsNotAssetOrBlueprint() && FoliageType->GetSource() == nullptr)
			{
				// We can't remove them here as we are within the loop itself so clean up after
				FoliageTypeToRemove.Add(FoliageType);
				
				continue;
			}

			// Upgrade foliage component
			if (GetLinkerCustomVersion(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::FoliageUsingFoliageISMC)
			{
				if (ensure(Info.Type == EFoliageImplType::StaticMesh))
				{
					FFoliageStaticMesh* FoliageStaticMesh = StaticCast<FFoliageStaticMesh*>(Info.Implementation.Get());
					UFoliageType_InstancedStaticMesh* FoliageType_InstancedStaticMesh = Cast<UFoliageType_InstancedStaticMesh>(FoliageType);
					FoliageStaticMesh->CheckComponentClass(FoliageType_InstancedStaticMesh);
				}
			}

			if (GetLinkerCustomVersion(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::FoliageActorSupportNoWeakPtr)
			{
				if (Info.Type == EFoliageImplType::Actor)
				{
					FFoliageActor* FoliageActor = StaticCast<FFoliageActor*>(Info.Implementation.Get());
					for (const TWeakObjectPtr<AActor>& ActorPtr : FoliageActor->ActorInstances_Deprecated)
					{
						FoliageActor->ActorInstances.Add(ActorPtr.Get());
					}
					FoliageActor->ActorInstances_Deprecated.Empty();
				}
			}

			// Fixup FoliageInfo instances at load
			// For Foliage meshes we compute the transforms based on its HISM instances transforms combined with IFA's transform
			// For Foliage actors we compute the transforms using spawned actors  transform
			if (GetLinkerCustomVersion(FFoliageCustomVersion::GUID) < FFoliageCustomVersion::FoliageRepairInstancesWithLevelTransform)
			{
				if (Info.Type == EFoliageImplType::StaticMesh)
				{
					FFoliageStaticMesh* FoliageStaticMesh = StaticCast<FFoliageStaticMesh*>(Info.Implementation.Get());
					if (FoliageStaticMesh->Component && (Info.Instances.Num() == FoliageStaticMesh->Component->PerInstanceSMData.Num()))
					{
						if (USceneComponent* IFARootComponent = this->GetRootComponent())
						{
							IFARootComponent->UpdateComponentToWorld();
							FTransform IFATransform = this->GetActorTransform();
							Info.InstanceHash->Empty();
							for (int32 InstanceIdx = 0; InstanceIdx < Info.Instances.Num(); InstanceIdx++)
							{
								FFoliageInstance& Instance = Info.Instances[InstanceIdx];
								FTransform Transform = FTransform(FoliageStaticMesh->Component->PerInstanceSMData[InstanceIdx].Transform)*IFATransform;
								Instance.Location = Transform.GetTranslation();
								Instance.Rotation = Transform.GetRotation().Rotator();
								// Rehash instance location
								Info.InstanceHash->InsertInstance(Instance.Location, InstanceIdx);
							}
						}
					}
				}
				else if (Info.Type == EFoliageImplType::Actor)
				{
					FFoliageActor* FoliageActor = StaticCast<FFoliageActor*>(Info.Implementation.Get());
					if (Info.Instances.Num() == FoliageActor->ActorInstances.Num())
					{
						Info.InstanceHash->Empty();
						for (int32 InstanceIdx = 0; InstanceIdx < Info.Instances.Num(); InstanceIdx++)
						{
							FFoliageInstance& Instance = Info.Instances[InstanceIdx];
							if (AActor* Actor = FoliageActor->ActorInstances[InstanceIdx])
							{
								Actor->ConditionalPostLoad();
								if (USceneComponent* ActorRootComponent = Actor->GetRootComponent())
								{
									ActorRootComponent->UpdateComponentToWorld();
									FTransform Transform = Actor->GetActorTransform();
									Instance.Location = Transform.GetTranslation();
									Instance.Rotation = Transform.GetRotation().Rotator();
								}
							}
							// Rehash instance location
							Info.InstanceHash->InsertInstance(Instance.Location, InstanceIdx);
						}
					}
				}
			}

			// Fixup corrupted data
			if (Info.Type == EFoliageImplType::StaticMesh)
			{
				UFoliageType_InstancedStaticMesh* FoliageType_InstancedStaticMesh = Cast<UFoliageType_InstancedStaticMesh>(FoliageType);
				if (UStaticMesh* FoliageTypeStaticMesh = FoliageType_InstancedStaticMesh->GetStaticMesh())
				{
					FFoliageStaticMesh* FoliageStaticMesh = StaticCast<FFoliageStaticMesh*>(Info.Implementation.Get());
					if (UHierarchicalInstancedStaticMeshComponent* HISMComponent = FoliageStaticMesh->Component)
					{
						HISMComponent->ConditionalPostLoad();
						UStaticMesh* ComponentStaticMesh = HISMComponent->GetStaticMesh();
						if (ComponentStaticMesh != FoliageTypeStaticMesh)
						{
							HISMComponent->SetStaticMesh(FoliageTypeStaticMesh);
						}
					}
				}
			}
		}

		UWorld* World = GetWorld();
		if (World && World->bIsWorldInitialized)
		{
			DetectFoliageTypeChangeAndUpdate();
		}

#if WITH_EDITORONLY_DATA
		if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::FoliageLazyObjPtrToSoftObjPtr)
		{
			for (auto Iter = InstanceBaseCache.InstanceBaseMap.CreateIterator(); Iter; ++Iter)
			{
				TPair<FFoliageInstanceBaseId, FFoliageInstanceBaseInfo>& Pair = *Iter;
				FFoliageInstanceBaseInfo& BaseInfo = Pair.Value;
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
			UActorComponent* Component = BaseInfo.BasePtr_DEPRECATED.Get();
				BaseInfo.BasePtr_DEPRECATED.Reset();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

				if (Component != nullptr)
				{
					BaseInfo.BasePtr = Component;

					if (!InstanceBaseCache.InstanceBaseInvMap.Contains(BaseInfo.BasePtr))
					{
						InstanceBaseCache.InstanceBaseInvMap.Add(BaseInfo.BasePtr, Pair.Key);
					}
				}
				else
				{
					Iter.RemoveCurrent();

					const FFoliageInstanceBasePtr* BaseInfoPtr = InstanceBaseCache.InstanceBaseInvMap.FindKey(Pair.Key);

					if (BaseInfoPtr != nullptr && BaseInfoPtr->Get() == nullptr)
					{
						InstanceBaseCache.InstanceBaseInvMap.Remove(*BaseInfoPtr);
					}
				}
			}

			InstanceBaseCache.InstanceBaseMap.Compact();
			InstanceBaseCache.InstanceBaseInvMap.Compact();

			for (auto& Pair : InstanceBaseCache.InstanceBaseLevelMap_DEPRECATED)
			{
				TArray<FFoliageInstanceBasePtr_DEPRECATED>& BaseInfo_DEPRECATED = Pair.Value;
				TArray<FFoliageInstanceBasePtr> BaseInfo;

				PRAGMA_DISABLE_DEPRECATION_WARNINGS
			for (FFoliageInstanceBasePtr_DEPRECATED& BasePtr_DEPRECATED : BaseInfo_DEPRECATED)
				{
					UActorComponent* Component = BasePtr_DEPRECATED.Get();
					BasePtr_DEPRECATED.Reset();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

					if (Component != nullptr)
					{
						BaseInfo.Add(Component);
					}
				}

				InstanceBaseCache.InstanceBaseLevelMap.Add(Pair.Key, BaseInfo);
			}

			InstanceBaseCache.InstanceBaseLevelMap_DEPRECATED.Empty();
		}

		// Clean up dead cross-level references
		FFoliageInstanceBaseCache::CompactInstanceBaseCache(this);
#endif

		// Clean up invalid foliage type
		for (UFoliageType* FoliageType : FoliageTypeToRemove)
		{
			OnFoliageTypeMeshChangedEvent.Broadcast(FoliageType);
			RemoveFoliageType(&FoliageType, 1);
		}
	}

#endif// WITH_EDITOR

	if (!GIsEditor)
	{
		const bool bFoliageDiscardOnLoad = !!CVarFoliageDiscardDataOnLoad.GetValueOnGameThread();
		const float FoliageCullDistanceScale = FMath::Clamp(CVarFoliageCullDistanceScale.GetValueOnGameThread(), 0.0f, 1.0f);

		bool bHasISMFoliage = false;
		for (auto& Pair : FoliageInfos)
		{
			if (bFoliageDiscardOnLoad && (!Pair.Key || Pair.Key->bEnableDiscardOnLoad))
			{
				if (Pair.Value->Type == EFoliageImplType::StaticMesh)
				{
					FFoliageStaticMesh* FoliageStaticMesh = StaticCast<FFoliageStaticMesh*>(Pair.Value->Implementation.Get());

					if (FoliageStaticMesh->Component != nullptr)
					{
						FoliageStaticMesh->Component->ConditionalPostLoad();
						FoliageStaticMesh->Component->DestroyComponent();
					}
				}
				else if (Pair.Value->Type == EFoliageImplType::Actor)
				{
					FFoliageActor* FoliageActor = StaticCast<FFoliageActor*>(Pair.Value->Implementation.Get());
					FoliageActor->DestroyActors(true);
				}
				else if (Pair.Value->Type == EFoliageImplType::ISMActor)
				{
					bHasISMFoliage = true;
				}

#if !WITH_EDITOR
				// If we are running in -game we still need foliage info for any remote transactional events.
				Pair.Value = FFoliageInfo();
#endif
			}
			else if (Pair.Key && Pair.Key->bEnableCullDistanceScaling)
			{
				if (Pair.Value->Type == EFoliageImplType::StaticMesh)
				{
					FFoliageStaticMesh* FoliageStaticMesh = StaticCast<FFoliageStaticMesh*>(Pair.Value->Implementation.Get());

					if (FoliageStaticMesh->Component != nullptr)
					{
						FoliageStaticMesh->Component->ConditionalPostLoad();
						FoliageUtil::UpdateComponentCullDistance(Pair.Key, FoliageStaticMesh->Component, FoliageCullDistanceScale);
					}
				}
			}
		}

		if (bHasISMFoliage)
		{
			TArray<UFoliageInstancedStaticMeshComponent*> FoliageComponents;
			GetComponents(FoliageComponents);
			for (UFoliageInstancedStaticMeshComponent* FoliageComponent : FoliageComponents)
			{
				if (FoliageComponent && FoliageComponent->bEnableDiscardOnLoad)
				{
					FoliageComponent->ConditionalPostLoad();
					FoliageComponent->DestroyComponent();
				}
			}
		}
	}

	for (auto& Pair : FoliageInfos)
	{
		Pair.Value->PostLoad();
	}
}

#if WITH_EDITORONLY_DATA
void AInstancedFoliageActor::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UFoliageInstancedStaticMeshComponent::StaticClass()));
}
#endif

#if WITH_EDITOR

void AInstancedFoliageActor::RepairDuplicateIFA(AInstancedFoliageActor* DuplicateIFA)
{
	for (auto& Pair : DuplicateIFA->FoliageInfos)
	{
		UFoliageType* DupeFoliageType = Pair.Key;
		FFoliageInfo& DupeMeshInfo = *Pair.Value;

		// Get foliage type compatible with target IFA
		FFoliageInfo* TargetMeshInfo = nullptr;
		UFoliageType* TargetFoliageType = AddFoliageType(DupeFoliageType, &TargetMeshInfo);

		// Copy the instances
		for (FFoliageInstance& Instance : DupeMeshInfo.Instances)
		{
			if ((Instance.Flags & FOLIAGE_InstanceDeleted) == 0)
			{
				TargetMeshInfo->AddInstance(TargetFoliageType, Instance);
			}
		}

		TargetMeshInfo->Refresh(true, true);
	}

	GetWorld()->DestroyActor(DuplicateIFA);
}

void AInstancedFoliageActor::NotifyFoliageTypeChanged(UFoliageType* FoliageType, bool bSourceChanged)
{
	FFoliageInfo* TypeInfo = FindInfo(FoliageType);

	if (TypeInfo)
	{
		TypeInfo->NotifyFoliageTypeChanged(FoliageType, bSourceChanged);	

		if (bSourceChanged)
		{
			// If the type's mesh has changed, the UI needs to be notified so it can update thumbnails accordingly
			OnFoliageTypeMeshChangedEvent.Broadcast(FoliageType);		

			if (FoliageType->IsNotAssetOrBlueprint() && FoliageType->GetSource() == nullptr) //If the source data has been deleted and we're a per foliage actor instance we must remove all instances 
			{
				RemoveFoliageType(&FoliageType, 1);
			}
		}
	}
}

void AInstancedFoliageActor::RemoveBaseComponentOnFoliageTypeInstances(UFoliageType* FoliageType)
{
	FFoliageInfo* TypeInfo = FindInfo(FoliageType);

	if (TypeInfo)
	{
		TypeInfo->RemoveBaseComponentOnInstances();
	}
}

void AInstancedFoliageActor::AddInstances(UObject* WorldContextObject, UFoliageType* InFoliageType, const TArray<FTransform>& InTransforms)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World)
	{
		TMap<AInstancedFoliageActor*, TArray<const FFoliageInstance*>> InstancesToAdd;
		TArray<FFoliageInstance> FoliageInstances;
		FoliageInstances.Reserve(InTransforms.Num()); // Reserve 

		for (const FTransform& InstanceTransfo : InTransforms)
		{
			AInstancedFoliageActor* IFA = AInstancedFoliageActor::Get(World, true, World->PersistentLevel, InstanceTransfo.GetLocation());
			FFoliageInstance FoliageInstance;
			FoliageInstance.Location = InstanceTransfo.GetLocation();
			FoliageInstance.Rotation = InstanceTransfo.GetRotation().Rotator();
			FoliageInstance.DrawScale3D = (FVector3f)InstanceTransfo.GetScale3D();

			FoliageInstances.Add(FoliageInstance);
			InstancesToAdd.FindOrAdd(IFA).Add(&FoliageInstances[FoliageInstances.Num() - 1]);
		}

		for (const auto& Pair : InstancesToAdd)
		{
			FFoliageInfo* TypeInfo = nullptr;
			if (UFoliageType* FoliageType = Pair.Key->AddFoliageType(InFoliageType, &TypeInfo))
			{
				TypeInfo->AddInstances(FoliageType, Pair.Value);
			}
		}
	}
}

void AInstancedFoliageActor::RemoveAllInstances(UObject* WorldContextObject, UFoliageType* InFoliageType)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World)
	{
		for (TActorIterator<AInstancedFoliageActor> It(World); It; ++It)
		{
			AInstancedFoliageActor* IFA = (*It);
			IFA->RemoveFoliageType(&InFoliageType, 1);
		}
	}
}

void AInstancedFoliageActor::NotifyFoliageTypeWillChange(UFoliageType* FoliageType)
{
	FFoliageInfo* TypeInfo = FindInfo(FoliageType);

	// Change bounds delegate bindings
	if (TypeInfo)
	{
		TypeInfo->NotifyFoliageTypeWillChange(FoliageType);
	}
}

void AInstancedFoliageActor::UpdateFoliageActorInstance(AActor* InActor)
{
	check(FFoliageHelper::IsOwnedByFoliage(InActor));

	TSet<int32> InstanceToMove;
	FFoliageInfo* OldFoliageInfo = nullptr;
	UFoliageType* FoliageType = nullptr;

	for (auto& Pair : FoliageInfos)
	{
		if (Pair.Value->Type == EFoliageImplType::Actor)
		{
			FFoliageActor* FoliageActor = StaticCast<FFoliageActor*>(Pair.Value->Implementation.Get());
			int32 Index = FoliageActor->FindIndex(InActor);
			if (Index != INDEX_NONE)
			{
				InstanceToMove.Add(Index);
				OldFoliageInfo = &Pair.Value.Get();
				FoliageType = Pair.Key;
				break;
			}
		}
	}

	if (InstanceToMove.Num() > 0)
	{
		MoveInstancesToLevel(InActor->GetLevel(), InstanceToMove, OldFoliageInfo, FoliageType, true);
	}
}

void AInstancedFoliageActor::DeleteFoliageActorInstance(AActor* InActor)
{
	check(FFoliageHelper::IsOwnedByFoliage(InActor));

	for (auto& Pair : FoliageInfos)
	{
		FFoliageInfo& Info = *Pair.Value;
		UFoliageType* FoliageType = Pair.Key;

		if (Info.Type == EFoliageImplType::Actor)
		{
			if (FFoliageActor* FoliageActor = StaticCast<FFoliageActor*>(Info.Implementation.Get()))
			{
				TArray<int32> InstancesToRemove;
				// Make sure we find Null pointers and Delete those instances if we can't find the actor
				FoliageActor->GetInvalidInstances(InstancesToRemove);
				// Find Actor
				int32 Index = FoliageActor->FindIndex(InActor);
				if (Index != INDEX_NONE)
				{
					InstancesToRemove.Add(Index);
				}

				if (InstancesToRemove.Num() > 0)
				{
					Info.RemoveInstances(InstancesToRemove, false);
					if (InstanceCountChanged.IsBound())
					{
						InstanceCountChanged.Broadcast(FoliageType);
					}
				}
			}
		}
	}
}

// This logic was extracted from AInstancedFoliageActor::PostLoad to be called once the World is done initializing
void AInstancedFoliageActor::DetectFoliageTypeChangeAndUpdate()
{
	for (auto& Pair : FoliageInfos)
	{
		// Find the per-mesh info matching the mesh.
		FFoliageInfo& Info = *Pair.Value;
		UFoliageType* FoliageType = Pair.Key;

		if (Info.FoliageTypeUpdateGuid != FoliageType->UpdateGuid)
		{
			if (Info.FoliageTypeUpdateGuid.IsValid())
			{
				// Update foliage component settings if the foliage settings object was changed while the level was not loaded.
				if (Info.Type == EFoliageImplType::StaticMesh)
				{
					FFoliageStaticMesh* FoliageStaticMesh = StaticCast<FFoliageStaticMesh*>(Info.Implementation.Get());
					UFoliageType_InstancedStaticMesh* FoliageType_InstancedStaticMesh = Cast<UFoliageType_InstancedStaticMesh>(FoliageType);
					FoliageStaticMesh->CheckComponentClass(FoliageType_InstancedStaticMesh);
					FoliageStaticMesh->UpdateComponentSettings(FoliageType_InstancedStaticMesh);
				}
				// Respawn foliage 
				else
				{
					// We can't spawn in postload because BeingPlay might call UnrealScript which is not supported.
					UWorld* World = GetWorld();
					if (World && !World->IsGameWorld())
					{
						Info.Implementation->NotifyFoliageTypeWillChange(FoliageType);

						EFoliageImplType CurrentType = Info.GetImplementationType(FoliageType);
						if (Info.Type != Info.GetImplementationType(FoliageType))
						{
							Info.Implementation->Uninitialize();
							Info.Implementation.Reset();
							Info.CreateImplementation(CurrentType);
							Info.Implementation->Reapply(FoliageType);
						}
						else
						{
							Info.Implementation->NotifyFoliageTypeChanged(FoliageType, false);
						}
					}
				}
			}
			Info.FoliageTypeUpdateGuid = FoliageType->UpdateGuid;
		}
	}
}

uint32 AInstancedFoliageActor::GetDefaultGridSize(UWorld* InWorld) const
{
	return InWorld->GetWorldSettings()->InstancedFoliageGridSize;
}

bool AInstancedFoliageActor::ShouldIncludeGridSizeInName(UWorld* InWorld, const FActorPartitionIdentifier& InIdentifier) const
{
	return InWorld->GetWorldSettings()->bIncludeGridSizeInNameForFoliageActors;
}

void AInstancedFoliageActor::PostApplyLevelTransform(const FTransform& InTransform)
{
#if WITH_EDITORONLY_DATA
	check(GIsEditor);
	InstanceBaseCache.UpdateInstanceBaseCachedTransforms();
	for (auto& Pair : FoliageInfos)
	{
		FFoliageInfo& Info = *Pair.Value;
		Info.InstanceHash->Empty();
		for (int32 InstanceIdx = 0; InstanceIdx < Info.Instances.Num(); InstanceIdx++)
		{
			FFoliageInstance& Instance = Info.Instances[InstanceIdx];
			FTransform OldTransform(Instance.Rotation, Instance.Location);
			FTransform NewTransform = OldTransform * InTransform;
			Instance.Location = NewTransform.GetTranslation();
			Instance.Rotation = NewTransform.Rotator();
			// Rehash instance location
			Info.InstanceHash->InsertInstance(Instance.Location, InstanceIdx);
		}
	}
#endif
}

void AInstancedFoliageActor::PostApplyLevelOffset(const FVector& InOffset, bool bWorldShift)
{
	check(GIsEditor);
	for (auto& Pair : FoliageInfos)
	{
		FFoliageInfo& Info = *Pair.Value;

		InstanceBaseCache.UpdateInstanceBaseCachedTransforms();

		Info.InstanceHash->Empty();
		for (int32 InstanceIdx = 0; InstanceIdx < Info.Instances.Num(); InstanceIdx++)
		{
			FFoliageInstance& Instance = Info.Instances[InstanceIdx];
			Instance.Location += InOffset;
			// Rehash instance location
			Info.InstanceHash->InsertInstance(Instance.Location, InstanceIdx);
		}
	}
}


void AInstancedFoliageActor::CleanupDeletedFoliageType()
{
	for (auto It = FoliageInfos.CreateIterator(); It; ++It)
	{
		if (It->Key == nullptr)
		{
			FFoliageInfo& Info = *It->Value;
			TArray<int32> InstancesToRemove;
			for (int32 InstanceIdx = 0; InstanceIdx < Info.Instances.Num(); InstanceIdx++)
			{
				InstancesToRemove.Add(InstanceIdx);
			}

			if (InstancesToRemove.Num())
			{
				Info.RemoveInstances(InstancesToRemove, true);
			}

			It.RemoveCurrent();
		}
	}
}


#endif
//
// Serialize all our UObjects for RTGC 
//
void AInstancedFoliageActor::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	AInstancedFoliageActor* This = CastChecked<AInstancedFoliageActor>(InThis);

	for (auto& Pair : This->FoliageInfos)
	{
		Collector.AddReferencedObject(Pair.Key, This);
		FFoliageInfo& Info = *Pair.Value;

		Info.AddReferencedObjects(This, Collector);
	}

	Super::AddReferencedObjects(This, Collector);
}

#if WITH_EDITOR
bool AInstancedFoliageActor::FoliageTrace(const UWorld* InWorld, FHitResult& OutHit, const FDesiredFoliageInstance& DesiredInstance, FName InTraceTag, bool InbReturnFaceIndex, const FFoliageTraceFilterFunc& FilterFunc, bool bAverageNormal)
{
	SCOPE_CYCLE_COUNTER(STAT_FoliageTrace);

	FCollisionQueryParams QueryParams(InTraceTag, SCENE_QUERY_STAT_ONLY(IFA_FoliageTrace), true);
	QueryParams.bReturnFaceIndex = InbReturnFaceIndex;

	//It's possible that with the radius of the shape we will end up with an initial overlap which would place the instance at the top of the procedural volume.
	//Moving the start trace back a bit will fix this, but it introduces the potential for spawning instances a bit above the volume. This second issue is already somewhat broken because of how sweeps work so it's not too bad, also this is a less common case.
	//The proper fix would be to do something like EncroachmentCheck where we first do a sweep, then we fix it up if it's overlapping, then check the filters. This is more expensive and error prone so for now we just move the trace up a bit.
	const FVector Dir = (DesiredInstance.EndTrace - DesiredInstance.StartTrace).GetSafeNormal();
	const FVector StartTrace = DesiredInstance.StartTrace - (Dir * DesiredInstance.TraceRadius);

	TArray<FHitResult> Hits;
	FCollisionShape SphereShape;
	SphereShape.SetSphere(DesiredInstance.TraceRadius);
	InWorld->SweepMultiByObjectType(Hits, StartTrace, DesiredInstance.EndTrace, FQuat::Identity, FCollisionObjectQueryParams(ECC_WorldStatic), SphereShape, QueryParams);
	auto ValidateHit = [&DesiredInstance, &FilterFunc](const FHitResult& Hit, FHitResult& OutHit, bool& bOutDiscardHit, bool& bOutInsideProceduralVolumeOrArentUsingOne)
	{
		bOutDiscardHit = false;
		bOutInsideProceduralVolumeOrArentUsingOne = false;
		const FActorInstanceHandle& HitObjectHandle = Hit.HitObjectHandle;

		// don't place procedural foliage inside an AProceduralFoliageBlockingVolume
		// this test is first because two of the tests below would otherwise cause the trace to ignore AProceduralFoliageBlockingVolume
		if (DesiredInstance.PlacementMode == EFoliagePlacementMode::Procedural)
		{
			if (const AProceduralFoliageBlockingVolume* ProceduralFoliageBlockingVolume = HitObjectHandle.FetchActor<AProceduralFoliageBlockingVolume>())
			{
				const AProceduralFoliageVolume* ProceduralFoliageVolume = ProceduralFoliageBlockingVolume->ProceduralFoliageVolume;
				if (ProceduralFoliageVolume == nullptr || ProceduralFoliageVolume->ProceduralComponent == nullptr || ProceduralFoliageVolume->ProceduralComponent->GetProceduralGuid() == DesiredInstance.ProceduralGuid)
				{
					if (!ProceduralFoliageBlockingVolume->DensityFalloff.bUseFalloffCurve)
					{
						return false;
					}
					else if (UBrushComponent* Brush = ProceduralFoliageBlockingVolume->GetBrushComponent())
					{
						FBox ActorVolumeBounds = Brush->Bounds.GetBox();
						FVector2D ActorVolumeLocation = FVector2D(ActorVolumeBounds.GetCenter());
						const FVector::FReal ActorVolumeMaxExtent = FVector2D(ActorVolumeBounds.GetExtent()).GetMax();

						const FVector2D Origin(ProceduralFoliageBlockingVolume->GetActorTransform().GetLocation());
						if (ProceduralFoliageBlockingVolume->DensityFalloff.IsInstanceFiltered(FVector2D(Hit.ImpactPoint), ActorVolumeLocation, ActorVolumeMaxExtent))
						{
							return false;
						}
					}
				}
			}
			else if (HitObjectHandle.IsValid() && HitObjectHandle.DoesRepresentClass(AProceduralFoliageVolume::StaticClass())) //we never want to collide with our spawning volume
			{
				bOutDiscardHit = true;
				return true;
			}
		}

		const UPrimitiveComponent* HitComponent = Hit.GetComponent();
		
		if (HitComponent)
		{
			// In the editor traces can hit "No Collision" type actors, so ugh. (ignore these)
			if (!HitComponent->IsQueryCollisionEnabled() || HitComponent->GetCollisionResponseToChannel(ECC_WorldStatic) != ECR_Block)
			{
				bOutDiscardHit = true;
				return true;
			}

			// Don't place foliage on invisible walls / triggers / volumes
			if (HitComponent->IsA<UBrushComponent>())
			{
				bOutDiscardHit = true;
				return true;
			}
		}

		// Don't place foliage on itself
		const AActor* HitActor = Hit.HitObjectHandle.FetchActor();
		const AInstancedFoliageActor* IFA = Cast<AInstancedFoliageActor>(HitActor);

		if (HitActor)
		{
			check(HitComponent);

			// Don't place foliage on hidden actors.
			if (HitActor->IsTemporarilyHiddenInEditor())
			{
				bOutDiscardHit = true;
				return true;
			}

			// Don't place foliage on itself.
			if (!IFA && FFoliageHelper::IsOwnedByFoliage(HitActor))
			{
				IFA = HitActor->GetLevel()->InstancedFoliageActor.Get();
				if (IFA == nullptr)
				{
					bOutDiscardHit = true;
					return true;
				}

				if (const FFoliageInfo* FoundMeshInfo = IFA->FindInfo(DesiredInstance.FoliageType))
				{
					if (FoundMeshInfo->Implementation->IsOwnedComponent(HitComponent))
					{
						bOutDiscardHit = true;
						return true;
					}
				}
			}
		}

		if (FilterFunc && FilterFunc(HitComponent) == false)
		{
			// supplied filter does not like this component, so keep iterating
			bOutDiscardHit = true;
			return true;
		}

		bOutInsideProceduralVolumeOrArentUsingOne = true;
		if (DesiredInstance.PlacementMode == EFoliagePlacementMode::Procedural && DesiredInstance.ProceduralVolumeBodyInstance)
		{
			// We have a procedural volume, so lets make sure we are inside it.
			bOutInsideProceduralVolumeOrArentUsingOne = DesiredInstance.ProceduralVolumeBodyInstance->OverlapTest(Hit.ImpactPoint, FQuat::Identity, FCollisionShape::MakeSphere(1.f));	//make sphere of 1cm radius to test if we're in the procedural volume
		}

		OutHit = Hit;
		
		// When placing foliage on other foliage, we need to return the base component of the other foliage, not the foliage component, so that it moves correctly
		if (IFA)
		{
			for (auto& Pair : IFA->FoliageInfos)
			{
				const FFoliageInfo& Info = *Pair.Value;
				int32 InstanceIndex = Info.Implementation->GetInstanceIndexFrom(HitComponent, Hit.Item);
				if (InstanceIndex != INDEX_NONE)
				{
					OutHit.Component = CastChecked<UPrimitiveComponent>(IFA->InstanceBaseCache.GetInstanceBasePtr(Info.Instances[InstanceIndex].BaseId).Get(), ECastCheckedType::NullAllowed);
					break;
				}
			}

			// The foliage we are snapping on doesn't have a valid base
			if (!OutHit.Component.IsValid())
			{
				bOutDiscardHit = true;
			}
		}

		return true;
	};

	for (const FHitResult& Hit : Hits)
	{
		bool bOutDiscardHit = false;
		bool bOutInsideProceduralVolumeOrArentUsingOne = false;
		if (!ValidateHit(Hit, OutHit, bOutDiscardHit, bOutInsideProceduralVolumeOrArentUsingOne))
		{
			return false;
		}
					
		if (bOutDiscardHit)
		{
			continue;
		}

		if (bAverageNormal && DesiredInstance.FoliageType && DesiredInstance.FoliageType->AverageNormal)
		{
			int32 PointSeed = FFoliagePlacementUtil::GetRandomSeedForPosition(FVector2D(Hit.Location));
			FRandomStream LocalRandomStream(PointSeed);
			TArray<FHitResult> NormalHits;
			FVector CumulativeNormal = OutHit.ImpactNormal;
			FHitResult OutNormalHit;
			bool bSingleComponent = DesiredInstance.FoliageType->AverageNormalSingleComponent;
			for (int32 i = 0; i < DesiredInstance.FoliageType->AverageNormalSampleCount; ++i)
			{
				const FVector::FReal Angle = LocalRandomStream.FRandRange(0, PI * 2.f);
				const FVector::FReal SqrtRadius = FMath::Sqrt(LocalRandomStream.FRand()) * DesiredInstance.FoliageType->LowBoundOriginRadius.Z;
				FVector Offset(SqrtRadius * FMath::Cos(Angle), SqrtRadius* FMath::Sin(Angle), 0.f);
				NormalHits.Reset();
				if (InWorld->LineTraceMultiByObjectType(NormalHits, StartTrace + Offset, DesiredInstance.EndTrace + Offset, FCollisionObjectQueryParams(ECC_WorldStatic), QueryParams))
				{
					for (const FHitResult& NormalHit : NormalHits)
					{
						bool bOutDiscardNormalHit = false;
						bool bIgnoredParam = false;

						if (ValidateHit(NormalHit, OutNormalHit, bOutDiscardNormalHit, bIgnoredParam))
						{
							if (!bOutDiscardNormalHit && (!bSingleComponent || OutNormalHit.Component == OutHit.Component))
							{
								CumulativeNormal += OutNormalHit.ImpactNormal;
								break;
							}
						}
					}
				}
			}
						
			OutHit.ImpactNormal = CumulativeNormal.GetSafeNormal();
		}
						
		return bOutInsideProceduralVolumeOrArentUsingOne;
	}

	return false;
}

bool AInstancedFoliageActor::CheckCollisionWithWorld(const UWorld* InWorld, const UFoliageType* Settings, const FFoliageInstance& Inst, const FVector& HitNormal, const FVector& HitLocation, UPrimitiveComponent* HitComponent)
{
	if (!Settings->CollisionWithWorld)
	{
		return true;
	}

	FTransform OriginalTransform = Inst.GetInstanceWorldTransform();
	OriginalTransform.SetRotation(FQuat::Identity);

	FMatrix InstTransformNoRotation = OriginalTransform.ToMatrixWithScale();
	OriginalTransform = Inst.GetInstanceWorldTransform();

	// Check for overhanging ledge
	const int32 SamplePositionCount = 4;
	{
		FVector LocalSamplePos[SamplePositionCount] = {
			FVector(Settings->LowBoundOriginRadius.Z, 0, 0),
			FVector(-Settings->LowBoundOriginRadius.Z, 0, 0),
			FVector(0, Settings->LowBoundOriginRadius.Z, 0),
			FVector(0, -Settings->LowBoundOriginRadius.Z, 0)
		};

		for (uint32 i = 0; i < SamplePositionCount; ++i)
		{
			FVector SamplePos = InstTransformNoRotation.TransformPosition(Settings->LowBoundOriginRadius + LocalSamplePos[i]);
			FVector::FReal WorldRadius = (Settings->LowBoundOriginRadius.Z + Settings->LowBoundOriginRadius.Z)*FMath::Max(Inst.DrawScale3D.X, Inst.DrawScale3D.Y);
			FVector NormalVector = Settings->AlignToNormal ? HitNormal : OriginalTransform.GetRotation().GetUpVector();

			//::DrawDebugSphere(InWorld, SamplePos, 10, 6, FColor::Red, true, 30.0f);
			//::DrawDebugSphere(InWorld, SamplePos - NormalVector*WorldRadius, 10, 6, FColor::Orange, true, 30.0f);
			//::DrawDebugDirectionalArrow(InWorld, SamplePos, SamplePos - NormalVector*WorldRadius, 10.0f, FColor::Red, true, 30.0f);

			FHitResult Hit;
			if (AInstancedFoliageActor::FoliageTrace(InWorld, Hit, FDesiredFoliageInstance(SamplePos, SamplePos - NormalVector*WorldRadius, Settings)))
			{
				FVector LocalHit = OriginalTransform.InverseTransformPosition(Hit.Location);
				
				if (LocalHit.Z - Inst.ZOffset < Settings->LowBoundOriginRadius.Z && Hit.Component.Get() == HitComponent)
				{
					//::DrawDebugSphere(InWorld, Hit.Location, 6, 6, FColor::Green, true, 30.0f);
					continue;
				}
			}

			//::DrawDebugSphere(InWorld, SamplePos, 6, 6, FColor::Cyan, true, 30.0f);

			return false;
		}
	}

	FBoxSphereBounds LocalBound(Settings->MeshBounds.GetBox());
	FBoxSphereBounds WorldBound = LocalBound.TransformBy(OriginalTransform);

	static FName NAME_FoliageCollisionWithWorld = FName("FoliageCollisionWithWorld");
	if (InWorld->OverlapBlockingTestByChannel(WorldBound.Origin, FQuat(Inst.Rotation), ECC_WorldStatic, FCollisionShape::MakeBox(LocalBound.BoxExtent * (FVector)Inst.DrawScale3D * Settings->CollisionScale), FCollisionQueryParams(NAME_FoliageCollisionWithWorld, false, HitComponent != nullptr ? HitComponent->GetOwner() : nullptr)))
	{
		return false;
	}

	//::DrawDebugBox(InWorld, WorldBound.Origin, LocalBound.BoxExtent * Inst.DrawScale3D * Settings->CollisionScale, FQuat(Inst.Rotation), FColor::Red, true, 30.f);

	return true;
}

FPotentialInstance::FPotentialInstance(FVector InHitLocation, FVector InHitNormal, UPrimitiveComponent* InHitComponent, float InHitWeight, const FDesiredFoliageInstance& InDesiredInstance)
	: HitLocation(InHitLocation)
	, HitNormal(InHitNormal)
	, HitComponent(InHitComponent)
	, HitWeight(InHitWeight)
	, DesiredInstance(InDesiredInstance)
{
}

bool FPotentialInstance::PlaceInstance(const UWorld* InWorld, const UFoliageType* Settings, FFoliageInstance& Inst, bool bSkipCollision)
{
	if (DesiredInstance.PlacementMode != EFoliagePlacementMode::Procedural)
	{
		Inst.DrawScale3D = Settings->GetRandomScale();
		Inst.ZOffset = Settings->ZOffset.Interpolate(FMath::FRand());
	}
	else
	{
		//Procedural foliage uses age to get the scale
		Inst.DrawScale3D = FVector3f(Settings->GetScaleForAge(DesiredInstance.Age));
		
		// Use a deterministic seed for the offset in Procedural placement so that offset is always the same for the same instance position
		FRandomStream LocalRandomStream(FFoliagePlacementUtil::GetRandomSeedForPosition(FVector2D(Inst.Location)));
		Inst.ZOffset = Settings->ZOffset.Interpolate(LocalRandomStream.FRand());
	}
	
	Inst.Location = HitLocation;

	if (DesiredInstance.PlacementMode != EFoliagePlacementMode::Procedural)
	{
		// Random yaw and optional random pitch up to the maximum
		Inst.Rotation = FRotator(FMath::FRand() * Settings->RandomPitchAngle, 0.f, 0.f);

		if (Settings->RandomYaw)
		{
			Inst.Rotation.Yaw = FMath::FRand() * 360.f;
		}
		else
		{
			Inst.Flags |= FOLIAGE_NoRandomYaw;
		}
	}
	else
	{
		Inst.Rotation = DesiredInstance.Rotation.Rotator();
		Inst.Flags |= FOLIAGE_NoRandomYaw;
	}


	if (Settings->AlignToNormal)
	{
		Inst.AlignToNormal(HitNormal, Settings->AlignMaxAngle);
	}

	// Apply the Z offset in local space
	if (FMath::Abs(Inst.ZOffset) > KINDA_SMALL_NUMBER)
	{
		Inst.Location = Inst.GetInstanceWorldTransform().TransformPosition(FVector(0, 0, Inst.ZOffset));
	}

	UModelComponent* ModelComponent = Cast<UModelComponent>(HitComponent);
	if (ModelComponent)
	{
		ABrush* BrushActor = ModelComponent->GetModel()->FindBrush((FVector3f)HitLocation);
		if (BrushActor)
		{
			HitComponent = BrushActor->GetBrushComponent();
		}
	}

	return bSkipCollision || AInstancedFoliageActor::CheckCollisionWithWorld(InWorld, Settings, Inst, HitNormal, HitLocation, HitComponent);
}
#endif

float AInstancedFoliageActor::InternalTakeRadialDamage(float Damage, struct FRadialDamageEvent const& RadialDamageEvent, class AController* EventInstigator, AActor* DamageCauser)
{
	// Radial damage scaling needs to be applied per instance so we don't do anything here
	return Damage;
}

#if WITH_EDITOR
bool AInstancedFoliageActor::CanEditFoliageInstance(const FFoliageInstanceId& InstanceId) const
{
	return true;
}

bool AInstancedFoliageActor::CanMoveFoliageInstance(const FFoliageInstanceId& InstanceId, const ETypedElementWorldType WorldType) const
{
	return true;
}

bool AInstancedFoliageActor::GetFoliageInstanceTransform(const FFoliageInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace) const
{
	const FFoliageInstance& FoliageInstance = InstanceId.GetInstanceChecked();
	OutInstanceTransform = FoliageInstance.GetInstanceWorldTransform();
	if (!bWorldSpace)
	{
		OutInstanceTransform = OutInstanceTransform.GetRelativeTransform(GetActorTransform());
	}
	return true;
}

bool AInstancedFoliageActor::SetFoliageInstanceTransform(const FFoliageInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace, bool bTeleport)
{
	Modify();

	const bool bNeedsLocalMove = !InstanceId.Info->MovingInstances.Contains(InstanceId.Index);
	if (bNeedsLocalMove)
	{
		HandleFoliageInstancePreMove(InstanceId);
	}

	FTransform WorldTransform = bWorldSpace ? InstanceTransform : InstanceTransform * GetActorTransform();
	FFoliageInstance& FoliageInstance = InstanceId.GetInstanceChecked();
	FoliageInstance.SetInstanceWorldTransform(WorldTransform);
	InstanceId.Info->SetInstanceWorldTransform(InstanceId.Index, WorldTransform, bTeleport);

	if (bNeedsLocalMove)
	{
		HandleFoliageInstancePostMove(InstanceId);
	}

	return true;
}

void AInstancedFoliageActor::NotifyFoliageInstanceMovementStarted(const FFoliageInstanceId& InstanceId)
{
	HandleFoliageInstancePreMove(InstanceId);
}

void AInstancedFoliageActor::NotifyFoliageInstanceMovementOngoing(const FFoliageInstanceId& InstanceId)
{
	InstanceId.Info->PostMoveInstances(MakeArrayView(&InstanceId.Index, 1), /*bFinished*/false);
	InstanceId.Info->PreMoveInstances(MakeArrayView(&InstanceId.Index, 1));
}

void AInstancedFoliageActor::NotifyFoliageInstanceMovementEnded(const FFoliageInstanceId& InstanceId)
{
	HandleFoliageInstancePostMove(InstanceId);
}

void AInstancedFoliageActor::NotifyFoliageInstanceSelectionChanged(const FFoliageInstanceId& InstanceId, const bool bIsSelected)
{
	InstanceId.Info->SelectInstances(bIsSelected, MakeArrayView(&InstanceId.Index, 1));
}

bool AInstancedFoliageActor::DeleteFoliageInstances(TArrayView<const FFoliageInstanceId> InstanceIds)
{
	TMap<FFoliageInfo*, TArray<int32>> FoliageInfoToIndices;
	for (const FFoliageInstanceId& InstanceId : InstanceIds)
	{
		TArray<int32>& FoliageInstanceIndices = FoliageInfoToIndices.FindOrAdd(InstanceId.Info);
		FoliageInstanceIndices.Add(InstanceId.Index);
	}

	for (const TPair<FFoliageInfo*, TArray<int32>>& FoliageInfoPair : FoliageInfoToIndices)
	{
		FFoliageInfo* FoliageInfo = FoliageInfoPair.Key;
		const TArray<int32>& FoliageInstanceIndices = FoliageInfoPair.Value;

		FoliageInfo->RemoveInstances(FoliageInstanceIndices, /*bRebuildFoliageTree*/true);
	}

	return true;
}

bool AInstancedFoliageActor::DuplicateFoliageInstances(TArrayView<const FFoliageInstanceId> InstanceIds, TArray<FFoliageInstanceId>& OutNewInstanceIds)
{
	struct FDuplicateFoliageInstanceData
	{
		UFoliageType* FoliageType = nullptr;
		TArray<int32> InstanceIndices;
	};

	TMap<FFoliageInfo*, FDuplicateFoliageInstanceData> FoliageInfoToDuplicationData;
	for (const FFoliageInstanceId& InstanceId : InstanceIds)
	{
		UFoliageType* FoliageType = GetFoliageTypeForInfo(InstanceId.Info);
		check(FoliageType);

		FDuplicateFoliageInstanceData& DuplicationData = FoliageInfoToDuplicationData.FindOrAdd(InstanceId.Info);
		DuplicationData.FoliageType = FoliageType;
		DuplicationData.InstanceIndices.Add(InstanceId.Index);
	}

	OutNewInstanceIds.Reset(InstanceIds.Num());
	for (const TPair<FFoliageInfo*, FDuplicateFoliageInstanceData>& FoliageInfoPair : FoliageInfoToDuplicationData)
	{
		FFoliageInfo* FoliageInfo = FoliageInfoPair.Key;
		const FDuplicateFoliageInstanceData& DuplicationData = FoliageInfoPair.Value;

		const int32 PrevNumInstances = FoliageInfo->Instances.Num();
		FoliageInfo->DuplicateInstances(DuplicationData.FoliageType, DuplicationData.InstanceIndices);
		check(FoliageInfo->Instances.Num() == PrevNumInstances + DuplicationData.InstanceIndices.Num());

		OutNewInstanceIds.Reserve(OutNewInstanceIds.Num() + DuplicationData.InstanceIndices.Num());
		for (int32 InstanceIndex = PrevNumInstances; InstanceIndex < FoliageInfo->Instances.Num(); ++InstanceIndex)
		{
			OutNewInstanceIds.Add(FFoliageInstanceId{ FoliageInfo, InstanceIndex });
		}
	}

	return true;
}

void AInstancedFoliageActor::HandleFoliageInstancePreMove(const FFoliageInstanceId& InstanceId)
{
	InstanceId.Info->PreMoveInstances(MakeArrayView(&InstanceId.Index, 1));
}

void AInstancedFoliageActor::HandleFoliageInstancePostMove(const FFoliageInstanceId& InstanceId)
{
	InstanceId.Info->PostMoveInstances(MakeArrayView(&InstanceId.Index, 1), /*bFinished*/true);

	// This instance may be been moved into a new world partition
	// Verify, and re-instance any existing typed elements if required
	{
		FScopedActorEditorContextFromActor Context(this);
		const FFoliageInstance& FoliageInstance = InstanceId.GetInstanceChecked();
		AInstancedFoliageActor* TargetIFA = AInstancedFoliageActor::Get(GetWorld(), /*bCreateIfNone*/true, GetLevel(), FoliageInstance.Location);
		if (TargetIFA != this)
		{
			// Add the new instance first, as we need both instances available for the re-instance to work
			// Note: TargetIFA can be null if the target is an unloaded cell, in which case we'll re-instance against a null handle and the existing instance will be removed
			TArray<FSMInstanceId> NewSMInstanceIds;
			if (TargetIFA)
			{
				TargetIFA->Modify();

				UFoliageType* FoliageType = GetFoliageTypeForInfo(InstanceId.Info);
				check(FoliageType);

				FFoliageInfo* NewFoliageInfo = nullptr;
				UFoliageType* NewFoliageType = TargetIFA->AddFoliageType(FoliageType, &NewFoliageInfo);

				NewFoliageInfo->AddInstance(NewFoliageType, FoliageInstance, GetBaseComponentFromBaseId(FoliageInstance.BaseId));
				NewFoliageInfo->Refresh(true, true);

				NewFoliageInfo->ForEachSMInstance(NewFoliageInfo->Instances.Num() - 1, [&NewSMInstanceIds](FSMInstanceId NewSMInstanceId)
				{
					NewSMInstanceIds.Add(NewSMInstanceId);
					return true;
				});
			}

			// Re-instance this existing element to the new element
			{
				TMap<FSMInstanceId, FSMInstanceId> ReplacementSMInstanceIds;
				{
					int32 SMInstanceIdIndex = 0;
					InstanceId.Info->ForEachSMInstance(InstanceId.Index, [&NewSMInstanceIds, &ReplacementSMInstanceIds, &SMInstanceIdIndex](FSMInstanceId OldSMInstanceId)
					{
						ReplacementSMInstanceIds.Add(OldSMInstanceId, NewSMInstanceIds.IsValidIndex(SMInstanceIdIndex) ? NewSMInstanceIds[SMInstanceIdIndex] : FSMInstanceId());
						++SMInstanceIdIndex;
						return true;
					});
				}
				UEngineElementsLibrary::ReplaceEditorSMInstanceElementHandles(ReplacementSMInstanceIds);
			}

			// Now remove the existing instance
			InstanceId.Info->RemoveInstances(MakeArrayView(&InstanceId.Index, 1), /*bRebuildFoliageTree*/true);
		}
	}
}

UFoliageType* AInstancedFoliageActor::GetFoliageTypeForInfo(const FFoliageInfo* FoliageInfo) const
{
	for (auto& Pair : FoliageInfos)
	{
		if (&Pair.Value.Get() == FoliageInfo)
		{
			return Pair.Key;
		}
	}
	return nullptr;
}
#endif

ISMInstanceManager* AInstancedFoliageActor::GetSMInstanceManager(const FSMInstanceId& InstanceId)
{
	if (!GIsEditor || !FoliageElementUtil::FoliageInstanceElementsEnabled())
	{
		return nullptr;
	}

#if WITH_EDITOR
	for (auto& Pair : FoliageInfos)
	{
		if (Pair.Value->Type == EFoliageImplType::StaticMesh && Pair.Value->Implementation->IsOwnedComponent(InstanceId.ISMComponent))
		{
			return static_cast<FFoliageStaticMesh*>(Pair.Value->Implementation.Get());
		}
	}
#endif

	return Super::GetSMInstanceManager(InstanceId);
}

UFoliageInstancedStaticMeshComponent::UFoliageInstancedStaticMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bEnableDiscardOnLoad(false)
{
	// Foliage is often built in world space which can cause problems with large world coordinates.
	// We use a translated space to deal with that.
	// todo: Maybe we can address this on the tool side and strip out all of the translated instance space code. It would require:
	// * foliage components to always be placed near their instances (instead of at origin).
	// * warnings and a fixup option for components which have instances with very large coordinates.
	bUseTranslatedInstanceSpace = true;

	bEnableAutoLODGeneration = false;

	ViewRelevanceType = EHISMViewRelevanceType::Foliage;
}

#if WITH_EDITOR
void UFoliageInstancedStaticMeshComponent::PreEditChange(FProperty* PropertyAboutToChange)
{
	// This component intentionally turns off the transaction flag when it is created and most changes go through the foliage editor.
	//
	// However, there are cases where users will modify the component directly and this allows us to capture those as proper transactions for undo/redo.  If a
	// component level change is made on a property this will temporarily enable the transaction flag so it passes checks in transaction code.
	//
	SetFlags(RF_Transactional);
	Super::PreEditChange(PropertyAboutToChange);
	ClearFlags(RF_Transactional);
}
#endif

void UFoliageInstancedStaticMeshComponent::ReceiveComponentDamage(float DamageAmount, FDamageEvent const& DamageEvent, AController* EventInstigator, AActor* DamageCauser)
{
	Super::ReceiveComponentDamage(DamageAmount, DamageEvent, EventInstigator, DamageCauser);

	if (DamageAmount != 0.f)
	{
		UDamageType const* const DamageTypeCDO = DamageEvent.DamageTypeClass ? DamageEvent.DamageTypeClass->GetDefaultObject<UDamageType>() : GetDefault<UDamageType>();
		if (DamageEvent.IsOfType(FPointDamageEvent::ClassID))
		{
			// Point damage event, hit a single instance.
			FPointDamageEvent* const PointDamageEvent = (FPointDamageEvent*)&DamageEvent;
			if (PerInstanceSMData.IsValidIndex(PointDamageEvent->HitInfo.Item))
			{
				OnInstanceTakePointDamage.Broadcast(PointDamageEvent->HitInfo.Item, DamageAmount, EventInstigator, PointDamageEvent->HitInfo.ImpactPoint, PointDamageEvent->ShotDirection, DamageTypeCDO, DamageCauser);
			}
		}
		else if (DamageEvent.IsOfType(FRadialDamageEvent::ClassID))
		{
			// Radial damage event, find which instances it hit and notify
			FRadialDamageEvent* const RadialDamageEvent = (FRadialDamageEvent*)&DamageEvent;

			float MaxRadius = RadialDamageEvent->Params.GetMaxRadius();
			TArray<int32> Instances = GetInstancesOverlappingSphere(RadialDamageEvent->Origin, MaxRadius, true);

			if (Instances.Num())
			{
				FVector LocalOrigin = GetComponentToWorld().Inverse().TransformPosition(RadialDamageEvent->Origin);
				FVector::FReal Scale = GetComponentScale().X; // assume component (not instances) is uniformly scaled

				TArray<float> Damages;
				Damages.Empty(Instances.Num());

				for (int32 InstanceIndex : Instances)
				{
					// Find distance in local space and then scale; quicker than transforming each instance to world space.
					float DistanceFromOrigin = static_cast<float>((PerInstanceSMData[InstanceIndex].Transform.GetOrigin() - LocalOrigin).Size() * Scale);
					Damages.Add(RadialDamageEvent->Params.GetDamageScale(DistanceFromOrigin));

				}

				OnInstanceTakeRadialDamage.Broadcast(Instances, Damages, EventInstigator, RadialDamageEvent->Origin, MaxRadius, DamageTypeCDO, DamageCauser);
			}
		}
	}
}

#if WITH_EDITOR

uint64 UFoliageInstancedStaticMeshComponent::GetHiddenEditorViews() const
{
	return FoliageHiddenEditorViews;
}

#endif// WITH_EDITOR

#undef LOCTEXT_NAMESPACE
