// Copyright Epic Games, Inc. All Rights Reserved.

#include "GrassInstancedStaticMeshComponent.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/StaticMesh.h"
#include "InstancedStaticMesh/ISMInstanceUpdateChangeSet.h"
#include "RHIGlobals.h"

UGrassInstancedStaticMeshComponent::UGrassInstancedStaticMeshComponent(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
	ViewRelevanceType = EHISMViewRelevanceType::Grass;
	// Tell the manager to not perform any tracking, and, indeed assert that none of the delta functions are ever called.
	PrimitiveInstanceDataManager.SetMode(FPrimitiveInstanceDataManager::EMode::ExternalLegacyData);
}

void UGrassInstancedStaticMeshComponent::BuildTreeAnyThread(
	TArray<FMatrix>& InstanceTransforms, 
	TArray<float>& InstanceCustomDataFloats,
	int32 NumCustomDataFloats,
	const FBox& MeshBox,
	TArray<FClusterNode>& OutClusterTree,
	TArray<int32>& OutSortedInstances,
	TArray<int32>& OutInstanceReorderTable,
	int32& OutOcclusionLayerNum,
	int32 MaxInstancesPerLeaf,
	bool InGenerateInstanceScalingRange
	)
{
	check(MaxInstancesPerLeaf > 0);

	// do grass need this?
	float DensityScaling = 1.0f;
	int32 InstancingRandomSeed = 1;

	FClusterBuilder Builder(InstanceTransforms, InstanceCustomDataFloats, NumCustomDataFloats, MeshBox, MaxInstancesPerLeaf, DensityScaling, InstancingRandomSeed, InGenerateInstanceScalingRange);
	Builder.BuildTree();
	OutOcclusionLayerNum = Builder.Result->OutOcclusionLayerNum;

	OutClusterTree = MoveTemp(Builder.Result->Nodes);
	OutInstanceReorderTable = MoveTemp(Builder.Result->InstanceReorderTable);
	OutSortedInstances = MoveTemp(Builder.Result->SortedInstances);
}

void UGrassInstancedStaticMeshComponent::AcceptPrebuiltTree(TArray<FClusterNode>& InClusterTree, int32 InOcclusionLayerNumNodes, int32 InNumBuiltRenderInstances, FStaticMeshInstanceData* InSharedInstanceBufferData)
{
	checkSlow(IsInGameThread());

	QUICK_SCOPE_CYCLE_COUNTER(STAT_UGrassInstancedStaticMeshComponent_AcceptPrebuiltTree);

	// this is only for prebuild data, already in the correct order
	check(!PerInstanceSMData.Num());

	NumBuiltInstances = 0;
	TranslatedInstanceSpaceOrigin = FVector::Zero();
	NumBuiltRenderInstances = InNumBuiltRenderInstances;
	check(NumBuiltRenderInstances);
	UnbuiltInstanceBounds.Init();
	UnbuiltInstanceBoundsList.Empty();
	ClusterTreePtr = MakeShareable(new TArray<FClusterNode>);
	InstanceReorderTable.Empty();
	SortedInstances.Empty();
	OcclusionLayerNumNodes = InOcclusionLayerNumNodes;
	BuiltInstanceBounds = GetClusterTreeBounds(InClusterTree, FVector::Zero());
	InstanceCountToRender = InNumBuiltRenderInstances;

	// Verify that the mesh is valid before using it.
	const bool bMeshIsValid =
		// make sure we have instances
		NumBuiltRenderInstances > 0 &&
		// make sure we have an actual staticmesh
		GetStaticMesh() &&
		GetStaticMesh()->HasValidRenderData();

	if (bMeshIsValid)
	{
		*ClusterTreePtr = MoveTemp(InClusterTree);

		PostBuildStats();

	}
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UGrassInstancedStaticMeshComponent_AcceptPrebuiltTree_Mark);

	check(PerInstanceSMData.Num() == 0 || InSharedInstanceBufferData->GetNumInstances() == PerInstanceSMData.Num());

	TUniquePtr<FStaticMeshInstanceData> BuiltInstanceData = MakeUnique<FStaticMeshInstanceData>();
	// TODO: Implement move semantics for FStaticMeshInstanceData!
	Swap(*BuiltInstanceData, *InSharedInstanceBufferData);
	PrimitiveInstanceDataManager.MarkForRebuildFromLegacy(MoveTemp(BuiltInstanceData), InstanceReorderTable, TArray<TRefCountPtr<HHitProxy>>());

	MarkRenderStateDirty();
}

void UGrassInstancedStaticMeshComponent::BuildTree()
{
	// Grass instances managed externally and handled in AcceptPrebuiltTree()
}

void UGrassInstancedStaticMeshComponent::BuildTreeAsync()
{
	// Grass instances managed externally and handled in AcceptPrebuiltTree()
}

// Deprecated 5.7
void UGrassInstancedStaticMeshComponent::BuildComponentInstanceData(ERHIFeatureLevel::Type FeatureLevel, FInstanceUpdateComponentDesc& OutData)
{
	return BuildComponentInstanceData(GetFeatureLevelShaderPlatform_Checked(FeatureLevel), OutData);
}

void UGrassInstancedStaticMeshComponent::BuildComponentInstanceData(EShaderPlatform InShaderPlatform, FInstanceUpdateComponentDesc& OutData)
{
	OutData.PrimitiveLocalToWorld = GetRenderMatrix();
	OutData.PrimitiveMaterialDesc = GetUsedMaterialPropertyDesc(InShaderPlatform);
	OutData.Flags = MakeInstanceDataFlags(OutData.PrimitiveMaterialDesc.bAnyMaterialHasPerInstanceRandom, OutData.PrimitiveMaterialDesc.bAnyMaterialHasPerInstanceCustomData);
	OutData.Flags.bHasPerInstanceDynamicData = false;
	OutData.StaticMeshBounds = GetStaticMesh()->GetBounds();
	OutData.NumProxyInstances = InstanceCountToRender;
	OutData.NumSourceInstances = 0;
	OutData.NumCustomDataFloats = NumCustomDataFloats;

#if WITH_EDITOR
	// TODO: Do we want these for this path?
	OutData.Flags.bHasPerInstanceEditorData = false;
#endif
	OutData.BuildChangeSet = [&](FISMInstanceUpdateChangeSet &ChangeSet)
	{
		// Cancel update as there is no source data
		check(ChangeSet.GetTransformDelta().IsEmpty());
		check(ChangeSet.GetCustomDataDelta().IsEmpty());

		BuildInstanceDataDeltaChangeSetCommon(ChangeSet);
		check(GetTranslatedInstanceSpaceOrigin().IsNearlyZero());
		check(PerInstanceSMData.IsEmpty());
		check(PerInstancePrevTransform.IsEmpty());
		// The reorder table is always empty in this path because the HISM is populated in AcceptPrebuiltTree using instances in the already sorted order.
		check(InstanceReorderTable.IsEmpty());
		check(ChangeSet.LegacyInstanceReorderTable.IsEmpty());
	};
}
