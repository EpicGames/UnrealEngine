// Copyright Epic Games, Inc. All Rights Reserved.

#include "InteractiveFoliageActor.h"
#include "Components/CapsuleComponent.h"
#include "Engine/CollisionProfile.h"
#include "InteractiveFoliageComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InteractiveFoliageActor)

AInteractiveFoliageActor::AInteractiveFoliageActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer
		.SetDefaultSubobjectClass<UInteractiveFoliageComponent>("StaticMeshComponent0"))
{

	UInteractiveFoliageComponent* FoliageMeshComponent = CastChecked<UInteractiveFoliageComponent>(GetStaticMeshComponent());
	FoliageMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	FoliageMeshComponent->Mobility = EComponentMobility::Static;

	CapsuleComponent = CreateDefaultSubobject<UCapsuleComponent>(TEXT("CollisionCylinder"));
	CapsuleComponent->InitCapsuleSize(60.0f, 200.0f);
	static FName CollisionProfileName("OverlapAllDynamic");
	CapsuleComponent->SetCollisionProfileName(CollisionProfileName);
	CapsuleComponent->Mobility = EComponentMobility::Static;

	RootComponent = CapsuleComponent;

	PrimaryActorTick.bCanEverTick = true;
	SetCanBeDamaged(true);
	bCollideWhenPlacing = true;
	SpawnCollisionHandlingMethod = ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButDontSpawnIfColliding;
	FoliageDamageImpulseScale = 20.0f;
	FoliageTouchImpulseScale = 10.0f;
	FoliageStiffness = 10.0f;
	FoliageStiffnessQuadratic = 0.3f;
	FoliageDamping = 2.0f;
	MaxDamageImpulse = 100000.0f;
	MaxTouchImpulse = 1000.0f;
	MaxForce = 100000.0f;
	Mass = 1.0f;
}

