// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/StaticMeshComponent.h"
#include "InteractiveFoliageComponent.generated.h"

class FPrimitiveSceneProxy;

UCLASS(hidecategories=Object)
class UInteractiveFoliageComponent : public UStaticMeshComponent
{
	GENERATED_UCLASS_BODY()


public:
	class FInteractiveFoliageSceneProxy* FoliageSceneProxy;

	//~ Begin UPrimitiveComponent Interface
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent Interface

	//~ Begin UActorComponent Interface
	virtual void DestroyRenderState_Concurrent() override;
	//~ End UActorComponent Interface

	friend class AInteractiveFoliageActor;
};

