// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralFoliageBlockingVolume.h"
#include "Components/BrushComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProceduralFoliageBlockingVolume)

static FName ProceduralFoliageBlocking_NAME("ProceduralFoliageBlockingVolume");

AProceduralFoliageBlockingVolume::AProceduralFoliageBlockingVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (UBrushComponent* MyBrushComponent = GetBrushComponent())
	{
		MyBrushComponent->SetCollisionObjectType(ECC_WorldStatic);
		MyBrushComponent->SetCollisionResponseToAllChannels(ECR_Ignore);
	}

	bIsEditorOnlyActor = true;
}

