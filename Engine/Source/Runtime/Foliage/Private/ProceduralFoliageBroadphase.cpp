// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralFoliageBroadphase.h"
#include "FoliageType_InstancedStaticMesh.h"

FProceduralFoliageBroadphase::FProceduralFoliageBroadphase(float TileSize, float MinimumQuadTreeSize)
	: QuadTree(FBox2D(FVector2D(-TileSize * 2.f, -TileSize * 2.f), FVector2D(TileSize * 2.f, TileSize * 2.f)), MinimumQuadTreeSize)
{
}

FProceduralFoliageBroadphase::FProceduralFoliageBroadphase(const FProceduralFoliageBroadphase& OtherBroadphase)
: QuadTree(FBox2D(FVector2D(0, 0), FVector2D(0, 0)))
{
	OtherBroadphase.QuadTree.Duplicate(QuadTree);
}

// Perhaps this should copy the QuadTree like the copy constructor does...?
FProceduralFoliageBroadphase& FProceduralFoliageBroadphase::operator=(const FProceduralFoliageBroadphase& OtherBroadphase) = default;

void FProceduralFoliageBroadphase::Empty()
{
	QuadTree.Empty();
}

/*Takes the instance and returns the AABB that contains both its shade and collision radius*/
FBox2D GetMaxAABB(FProceduralFoliageInstance* Instance)
{
	const float Radius = Instance->GetMaxRadius();
	const FVector2D Location(Instance->Location);
	const FVector2D Offset(Radius, Radius);
	const FBox2D AABB(Location - Offset, Location + Offset);
	return AABB;
}

bool FProceduralFoliageBroadphase::TestAgainstAABB(FProceduralFoliageInstance* Instance)
{
	const FBox2D MaxAABB = GetMaxAABB(Instance);
	return MaxAABB.Intersect(QuadTree.GetTreeBox());
}

void FProceduralFoliageBroadphase::Insert(FProceduralFoliageInstance* Instance)
{
	const FBox2D MaxAABB = GetMaxAABB(Instance);
	QuadTree.Insert(Instance, MaxAABB);
}

bool CircleOverlap(const FVector& ALocation, float ARadius, const FVector& BLocation, float BRadius)
{
	return (ALocation - BLocation).SizeSquared2D() <= (ARadius + BRadius)*(ARadius + BRadius);
}

bool FProceduralFoliageBroadphase::GetOverlaps(FProceduralFoliageInstance* Instance, TArray<FProceduralFoliageOverlap>& Overlaps) const
{
	const float AShadeRadius     = Instance->GetShadeRadius();
	const float ACollisionRadius = Instance->GetCollisionRadius();

	TArray<FProceduralFoliageInstance*> PossibleOverlaps;
	const FBox2D AABB = GetMaxAABB(Instance);
	QuadTree.GetElements(AABB, PossibleOverlaps);
	Overlaps.Reserve(Overlaps.Num() + PossibleOverlaps.Num());
	
	for (FProceduralFoliageInstance* Overlap : PossibleOverlaps)
	{
		if (Overlap != Instance)
		{
			//We must determine if this is an overlap of shade or an overlap of collision. If both the collision overlap wins
			bool bCollisionOverlap = CircleOverlap(Instance->Location, ACollisionRadius, Overlap->Location, Overlap->GetCollisionRadius());
			bool bShadeOverlap     = CircleOverlap(Instance->Location, AShadeRadius, Overlap->Location, Overlap->GetShadeRadius());

			if (bCollisionOverlap || bShadeOverlap)
			{
				new (Overlaps)FProceduralFoliageOverlap(Instance, Overlap, bCollisionOverlap ? ESimulationOverlap::CollisionOverlap : ESimulationOverlap::ShadeOverlap);
			}
			
		}
	}

	return Overlaps.Num() > 0;
}

void FProceduralFoliageBroadphase::Remove(FProceduralFoliageInstance* Instance)
{
	const FBox2D AABB = GetMaxAABB(Instance);
	const bool bRemoved = QuadTree.Remove(Instance, AABB);
	check(bRemoved);
}

void FProceduralFoliageBroadphase::GetInstancesInBox(const FBox2D& Box, TArray<FProceduralFoliageInstance*>& Instances) const
{
	QuadTree.GetElements(Box, Instances);
}
