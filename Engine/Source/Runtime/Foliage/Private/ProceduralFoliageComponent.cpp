// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralFoliageComponent.h"
#include "Async/Future.h"
#include "Async/Async.h"
#include "GameFramework/Volume.h"
#include "Components/BrushComponent.h"
#include "InstancedFoliageActor.h"
#include "ProceduralFoliageTile.h"
#include "ProceduralFoliageSpawner.h"
#include "Engine/LevelBounds.h"
#include "EngineUtils.h"
#include "Misc/FeedbackContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProceduralFoliageComponent)

#if WITH_EDITOR
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"
#endif

#define LOCTEXT_NAMESPACE "ProceduralFoliage"

UProceduralFoliageComponent::UProceduralFoliageComponent(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	TileOverlap = 0.f;
	if(!IsTemplate())
	{
		ProceduralGuid = FGuid::NewGuid();
	}
#if WITH_EDITORONLY_DATA
	bAllowLandscape = true;
	bAllowBSP = true;
	bAllowStaticMesh = true;
	bAllowTranslucent = false;
	bAllowFoliage = false;
	bShowDebugTiles = false;
#endif
}

static FBox2D GetTileRegion(const int32 X, const int32 Y, const float InnerSize, const float Overlap)
{
	// For tiles on the bottom and the left (that have no preceding neighbor to overlap them), have the
	// tile fill that space with its own content
	const float BottomLeftX = X == 0 ? -Overlap : Overlap;
	const float BottomLeftY = Y == 0 ? -Overlap : Overlap;
	FBox2D Region(FVector2D(BottomLeftX, BottomLeftY), FVector2D(InnerSize + Overlap, InnerSize + Overlap));

	return Region;
}

FBox UProceduralFoliageComponent::GetBounds() const
{
	UBrushComponent* Brush = SpawningVolume ? SpawningVolume->GetBrushComponent() : nullptr;
	if (Brush)
	{
		return Brush->Bounds.GetBox();
	}
	else
	{
		AActor* LocalOwner = GetOwner();
		ULevel* Level = LocalOwner ? LocalOwner->GetLevel() : nullptr;
		ALevelBounds* LevelBoundsActor = Level ? Level->LevelBoundsActor.Get() : nullptr;
		if (LevelBoundsActor)
		{
			return LevelBoundsActor->GetComponentsBoundingBox(false);
		}
	}

	return FBox(ForceInitToZero);
}

FBodyInstance* UProceduralFoliageComponent::GetBoundsBodyInstance() const
{
	UBrushComponent* Brush = SpawningVolume ? SpawningVolume->GetBrushComponent() : nullptr;
	if (Brush)
	{
		return Brush->GetBodyInstance();
	}

	return nullptr;
}

UProceduralFoliageComponent::FGenerateProceduralContentParams UProceduralFoliageComponent::GetGenerateProceduralContentParams() const
{
	FGenerateProceduralContentParams Params;
	Params.Bounds = GetBounds();
	Params.FoliageSpawner = FoliageSpawner;
	Params.ProceduralGuid = GetProceduralGuid();
	Params.TileOverlap = TileOverlap;
	Params.ProceduralVolumeInstance = GetBoundsBodyInstance();

	return Params;
}

void UProceduralFoliageComponent::GetTileLayout(const FGenerateProceduralContentParams& InParams, FTileLayout& OutTileLayout)
{
	if (InParams.Bounds.IsValid)
	{
		// Determine the bottom-left-most tile that contains the min position (when accounting for overlap)
		const FVector MinPosition = InParams.Bounds.Min + InParams.TileOverlap;
		OutTileLayout.BottomLeftX = FMath::FloorToInt(FloatCastChecked<float>(MinPosition.X / InParams.FoliageSpawner->TileSize, UE::LWC::DefaultFloatPrecision));
		OutTileLayout.BottomLeftY = FMath::FloorToInt(FloatCastChecked<float>(MinPosition.Y / InParams.FoliageSpawner->TileSize, UE::LWC::DefaultFloatPrecision));

		// Determine the total number of tiles along each active axis
		const FVector MaxPosition = InParams.Bounds.Max - InParams.TileOverlap;
		const int32 MaxXIdx = FMath::FloorToInt(FloatCastChecked<float>(MaxPosition.X / InParams.FoliageSpawner->TileSize, UE::LWC::DefaultFloatPrecision));
		const int32 MaxYIdx = FMath::FloorToInt(FloatCastChecked<float>(MaxPosition.Y / InParams.FoliageSpawner->TileSize, UE::LWC::DefaultFloatPrecision));

		OutTileLayout.NumTilesX = (MaxXIdx - OutTileLayout.BottomLeftX) + 1;
		OutTileLayout.NumTilesY = (MaxYIdx - OutTileLayout.BottomLeftY) + 1;

		OutTileLayout.HalfHeight = InParams.Bounds.GetExtent().Z;
	}
}

void UProceduralFoliageComponent::GetTileLayout(FTileLayout& OutTileLayout) const
{
	GetTileLayout(GetGenerateProceduralContentParams(), OutTileLayout);
}

#if WITH_EDITOR
void UProceduralFoliageComponent::LoadSimulatedRegion()
{
	if (GetOwner()->Implements<UWorldPartitionActorLoaderInterface>())
	{
		if (IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = Cast<IWorldPartitionActorLoaderInterface>(GetOwner())->GetLoaderAdapter())
		{
			LoaderAdapter->Load();
		}
	}
}

void UProceduralFoliageComponent::UnloadSimulatedRegion()
{
	if (GetOwner()->Implements<UWorldPartitionActorLoaderInterface>())
	{
		if (IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = Cast<IWorldPartitionActorLoaderInterface>(GetOwner())->GetLoaderAdapter())
		{
			LoaderAdapter->Unload();
		}
	}
}

bool UProceduralFoliageComponent::IsSimulatedRegionLoaded()
{
	if (GetOwner()->Implements<UWorldPartitionActorLoaderInterface>())
	{
		if (IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = Cast<IWorldPartitionActorLoaderInterface>(GetOwner())->GetLoaderAdapter())
		{
			return LoaderAdapter->IsLoaded();
		}
	}
	return true;
}
#endif

FVector UProceduralFoliageComponent::GetWorldPosition() const
{
	return GetWorldPosition(GetGenerateProceduralContentParams());
}

FVector UProceduralFoliageComponent::GetWorldPosition(const FGenerateProceduralContentParams& InParams)
{
	if (!InParams.Bounds.IsValid || InParams.FoliageSpawner == nullptr)
	{
		return FVector::ZeroVector;
	}
	
	FTileLayout TileLayout;
	GetTileLayout(InParams, TileLayout);

	const float TileSize = InParams.FoliageSpawner->TileSize;
	return FVector(TileLayout.BottomLeftX * TileSize, TileLayout.BottomLeftY * TileSize, InParams.Bounds.GetCenter().Z);
}

bool UProceduralFoliageComponent::GenerateProceduralContent(TArray<FDesiredFoliageInstance>& OutInstances)
{
#if WITH_EDITOR
	LoadSimulatedRegion();
	return GenerateProceduralContent(GetGenerateProceduralContentParams(), OutInstances);
#else
	return false;
#endif
}

bool UProceduralFoliageComponent::GenerateProceduralContent(const FGenerateProceduralContentParams& InParams, TArray<FDesiredFoliageInstance>& OutInstances)
{
#if WITH_EDITOR
	if (InParams.FoliageSpawner)
	{
		// Establish the counter used to check if the user has canceled the simulation
		FThreadSafeCounter LastCancel;
		const int32 LastCanelInit = LastCancel.GetValue();
		FThreadSafeCounter* LastCancelPtr = &LastCancel;
		
		// Establish basic info about the tiles
		const float TileOverlap = InParams.TileOverlap;
		const float TileSize = InParams.FoliageSpawner->TileSize;
		const FVector WorldPosition = GetWorldPosition(InParams);
		FVector2D GenerateLocation = FVector2D(InParams.Bounds.GetCenter());
		const FVector::FReal GenerateMaxExtent = FVector2D(InParams.Bounds.GetExtent()).GetMax();
		FTileLayout TileLayout;
		GetTileLayout(InParams, TileLayout);

		InParams.FoliageSpawner->Simulate();

		TArray<TFuture< TArray<FDesiredFoliageInstance>* >> Futures;
		for (int32 X = 0; X < TileLayout.NumTilesX; ++X)
		{
			for (int32 Y = 0; Y < TileLayout.NumTilesY; ++Y)
			{
				// We have to get the simulated tiles and create new ones to build on main thread
				const UProceduralFoliageTile* Tile = InParams.FoliageSpawner->GetRandomTile(X + TileLayout.BottomLeftX, Y + TileLayout.BottomLeftY);
				if (Tile == nullptr)	
				{
					// Simulation was either canceled or failed
					return false;
				}

				// From the pool of simulated tiles, pick the neighbors of this tile
				const UProceduralFoliageTile* RightTile = (X + 1 < TileLayout.NumTilesX) ? InParams.FoliageSpawner->GetRandomTile(X + TileLayout.BottomLeftX + 1, Y + TileLayout.BottomLeftY) : nullptr;
				const UProceduralFoliageTile* TopTile = (Y + 1 < TileLayout.NumTilesY) ? InParams.FoliageSpawner->GetRandomTile(X + TileLayout.BottomLeftX, Y + TileLayout.BottomLeftY + 1) : nullptr;
				const UProceduralFoliageTile* TopRightTile = (RightTile && TopTile) ? InParams.FoliageSpawner->GetRandomTile(X + TileLayout.BottomLeftX + 1, Y + TileLayout.BottomLeftY + 1) : nullptr;

				// Create a temp tile that will contain the composite contents of the tile after accounting for overlap
				UProceduralFoliageTile* CompositeTile = InParams.FoliageSpawner->CreateTempTile();

				Futures.Add(Async(EAsyncExecution::ThreadPool, [=]()
				{
					if (LastCancelPtr->GetValue() != LastCanelInit)
					{
						// The counter has changed since we began, meaning the user canceled the operation
						return new TArray<FDesiredFoliageInstance>();
					}

					//@todo proc foliage: Determine the composite contents of the tile (including overlaps) without copying everything to a temp tile

					// Copy the base tile contents
					const FBox2D BaseTile = GetTileRegion(X, Y, TileSize, InParams.TileOverlap);
					Tile->CopyInstancesToTile(CompositeTile, BaseTile, FTransform::Identity, TileOverlap);


					if (RightTile)
					{
						// Add instances from the right overlapping tile
						const FBox2D RightBox(FVector2D(0.f, BaseTile.Min.Y), FVector2D(TileOverlap, BaseTile.Max.Y));
						const FTransform RightTM(FVector(TileSize, 0.f, 0.f));
						RightTile->CopyInstancesToTile(CompositeTile, RightBox, RightTM, TileOverlap);
					}

					if (TopTile)
					{
						// Add instances from the top overlapping tile
						const FBox2D TopBox(FVector2D(BaseTile.Min.X, -TileOverlap), FVector2D(BaseTile.Max.X, TileOverlap));
						const FTransform TopTM(FVector(0.f, TileSize, 0.f));
						TopTile->CopyInstancesToTile(CompositeTile, TopBox, TopTM, TileOverlap);
					}

					if (TopRightTile)
					{
						// Add instances from the top-right overlapping tile
						const FBox2D TopRightBox(FVector2D(-TileOverlap, -TileOverlap), FVector2D(TileOverlap, TileOverlap));
						const FTransform TopRightTM(FVector(TileSize, TileSize, 0.f));
						TopRightTile->CopyInstancesToTile(CompositeTile, TopRightBox, TopRightTM, TileOverlap);
					}

					const FVector OrientedOffset = FVector(X, Y, 0.f) * TileSize;
					const FTransform TileTM(OrientedOffset + WorldPosition);

					TArray<FDesiredFoliageInstance>* DesiredInstances = new TArray<FDesiredFoliageInstance>();
					CompositeTile->ExtractDesiredInstances(*DesiredInstances, TileTM, GenerateLocation, GenerateMaxExtent, InParams.ProceduralGuid, TileLayout.HalfHeight, InParams.ProceduralVolumeInstance, true);

					return DesiredInstances;
					
				})
				);
			}
		}

		const FText StatusMessage = LOCTEXT("PlaceProceduralFoliage", "Placing ProceduralFoliage...");
		const FText CancelMessage = LOCTEXT("PlaceProceduralFoliageCancel", "Canceling ProceduralFoliage...");
		GWarn->BeginSlowTask(StatusMessage, true, true);


		int32 FutureIdx = 0;
		bool bCancelled = false;
		uint32 OutInstanceGrowth = 0;
		for (int X = 0; X < TileLayout.NumTilesX; ++X)
		{
			for (int Y = 0; Y < TileLayout.NumTilesY; ++Y)
			{
				bool bFirstTime = true;
				while (Futures[FutureIdx].WaitFor(FTimespan::FromMilliseconds(100.0)) == false || bFirstTime)
				{
					if (GWarn->ReceivedUserCancel() && bCancelled == false)
					{
						// Increment the thread-safe counter. Tiles compare against the original count and cancel if different.
						LastCancel.Increment();
						bCancelled = true;
					}

					if (bCancelled)
					{
						GWarn->StatusUpdate(Y + X * TileLayout.NumTilesY, TileLayout.NumTilesX * TileLayout.NumTilesY, CancelMessage);
					}
					else
					{
						GWarn->StatusUpdate(Y + X * TileLayout.NumTilesY, TileLayout.NumTilesX * TileLayout.NumTilesY, StatusMessage);
					}

					bFirstTime = false;
				}

				TArray<FDesiredFoliageInstance>* DesiredInstances = Futures[FutureIdx++].Get();
				OutInstanceGrowth += DesiredInstances->Num();
			}
		}

		OutInstances.Reserve(OutInstances.Num() + OutInstanceGrowth);
		FutureIdx = 0;
		for (int X = 0; X < TileLayout.NumTilesX; ++X)
		{
			for (int Y = 0; Y < TileLayout.NumTilesY; ++Y)
			{
				TArray<FDesiredFoliageInstance>* DesiredInstances = Futures[FutureIdx++].Get();
				OutInstances.Append(MoveTemp(*DesiredInstances));
				delete DesiredInstances;
			}
		}

		GWarn->EndSlowTask();

		return !bCancelled;
	}
#endif
	return false;

}

void UProceduralFoliageComponent::PostEditImport()
{
	// The Guid should always be unique
	ProceduralGuid = FGuid::NewGuid();
}

bool UProceduralFoliageComponent::ResimulateProceduralFoliage(TFunctionRef<void(const TArray<FDesiredFoliageInstance>&)> AddInstancesFunc)
{
#if WITH_EDITOR
	if (FoliageSpawner)
	{
		TArray<FDesiredFoliageInstance> DesiredFoliageInstances;
		if (GenerateProceduralContent(DesiredFoliageInstances))
		{
			if (DesiredFoliageInstances.Num() > 0)
			{
				{
					// Remove old foliage instances
					RemoveProceduralContent(false);
				}

				{
					// Add new foliage instances
					AddInstancesFunc(DesiredFoliageInstances);
				}
			}

			return true;
		}
	}
#endif

	return false;
}

void UProceduralFoliageComponent::RemoveProceduralContent(bool bInRebuildTree)
{
#if WITH_EDITOR
	TSet<AInstancedFoliageActor*> OutModifiedActors;
	RemoveProceduralContent(GetWorld(), GetProceduralGuid(), bInRebuildTree, OutModifiedActors);
#endif
}

void UProceduralFoliageComponent::RemoveProceduralContent(UWorld* InWorld, const FGuid& InProceduralGuid, bool bInRebuildTree, TSet<AInstancedFoliageActor*>& OutModifiedActors)
{
#if WITH_EDITOR
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		if ((*It)->DeleteInstancesForProceduralFoliageComponent(InProceduralGuid, bInRebuildTree))
		{
			OutModifiedActors.Add(*It);
		}
	}
#endif
}

bool UProceduralFoliageComponent::HasSpawnedAnyInstances()
{
#if WITH_EDITOR
	for (TActorIterator<AInstancedFoliageActor> It(GetWorld()); It; ++It)
	{
		if ((*It)->ContainsInstancesFromProceduralFoliageComponent(this))
		{
			return true;
		}
	}
#endif
	return false;
}

#undef LOCTEXT_NAMESPACE

