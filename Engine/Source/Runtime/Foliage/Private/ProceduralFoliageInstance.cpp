// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralFoliageInstance.h"
#include "FoliageType_InstancedStaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProceduralFoliageInstance)

FProceduralFoliageInstance::FProceduralFoliageInstance()
: Rotation(ForceInit)
, Location(ForceInit)
, Age(0)
, Normal(ForceInit)
, Scale(1)
, Type(nullptr)
, BaseComponent(nullptr)
, bBlocker(false)
, bAlive(true)
{
}

FProceduralFoliageInstance* GetLessFit(FProceduralFoliageInstance* A, FProceduralFoliageInstance* B)
{
	//Blocker is used for culling instances when we overlap tiles. It always wins
	if (A->bBlocker){ return B; }
	if (B->bBlocker){ return A; }
	
	//we look at priority, then age, then radius
	if (A->Type->OverlapPriority == B->Type->OverlapPriority)
	{
		if (A->Age == B->Age)
		{
			return A->Scale < B->Scale ? A : B;
		}
		else
		{
			return A->Age < B->Age ? A : B;
		}
	}
	else
	{
		return A->Type->OverlapPriority < B->Type->OverlapPriority ? A : B;
	}
}


FProceduralFoliageInstance* FProceduralFoliageInstance::Domination(FProceduralFoliageInstance* A, FProceduralFoliageInstance* B, ESimulationOverlap::Type OverlapType)
{
	const UFoliageType* AType = A->Type;
	const UFoliageType* BType = B->Type;

	FProceduralFoliageInstance* Dominated = GetLessFit(A, B);

	if (OverlapType == ESimulationOverlap::ShadeOverlap && Dominated->Type->bCanGrowInShade)
	{
		return nullptr;
	}

	return Dominated;
}

float FProceduralFoliageInstance::GetMaxRadius() const
{
	const float CollisionRadius = GetCollisionRadius();
	const float ShadeRadius = GetShadeRadius();
	return FMath::Max(CollisionRadius, ShadeRadius);
}

float FProceduralFoliageInstance::GetShadeRadius() const
{
	const float ShadeRadius = Type->ShadeRadius * Scale;
	return ShadeRadius;
}

float FProceduralFoliageInstance::GetCollisionRadius() const
{
	const float CollisionRadius = Type->CollisionRadius * Scale;
	return CollisionRadius;
}

void FProceduralFoliageInstance::TerminateInstance()
{
	bAlive = false;
}

