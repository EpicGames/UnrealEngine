// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralFoliageSpawner.h"
#include "ProceduralFoliageTile.h"
#include "ProceduralFoliageCustomVersion.h"
#include "Misc/FeedbackContext.h"
#include "Serialization/CustomVersion.h"
#include "Async/Async.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProceduralFoliageSpawner)

#define LOCTEXT_NAMESPACE "ProceduralFoliage"

UProceduralFoliageSpawner::UProceduralFoliageSpawner(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	TileSize = 10000;	//100 m
	MinimumQuadTreeSize = 100.f;
	NumUniqueTiles = 10;
	RandomSeed = 42;
}

UProceduralFoliageTile* UProceduralFoliageSpawner::CreateTempTile()
{
	UProceduralFoliageTile* TmpTile = NewObject<UProceduralFoliageTile>(this);
	TmpTile->InitSimulation(this, 0);

	return TmpTile;
}

void UProceduralFoliageSpawner::CreateProceduralFoliageInstances()
{
	for (FFoliageTypeObject& FoliageTypeObject : FoliageTypes)
	{
		// Refresh the instances contained in the type objects
		FoliageTypeObject.RefreshInstance();
	}
}

void UProceduralFoliageSpawner::SetClean()
{
	for (FFoliageTypeObject& FoliageTypeObject : FoliageTypes)
	{
		FoliageTypeObject.SetClean();
	}
}

const FGuid FProceduralFoliageCustomVersion::GUID(0xaafe32bd, 0x53954c14, 0xb66a5e25, 0x1032d1dd);
// Register the custom version with core
FCustomVersionRegistration GRegisterProceduralFoliageCustomVersion(FProceduralFoliageCustomVersion::GUID, FProceduralFoliageCustomVersion::LatestVersion, TEXT("ProceduralFoliageVer"));

void UProceduralFoliageSpawner::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FProceduralFoliageCustomVersion::GUID);
}

void UProceduralFoliageSpawner::Empty()
{
	for (TWeakObjectPtr<UProceduralFoliageTile>& WeakTile : PrecomputedTiles)
	{
		if (UProceduralFoliageTile* Tile = WeakTile.Get())
		{
			Tile->Empty();
		}
	}

	PrecomputedTiles.Empty();
}

const UProceduralFoliageTile* UProceduralFoliageSpawner::GetRandomTile(int32 X, int32 Y)
{
	if (PrecomputedTiles.Num())
	{
		// Random stream to use as a hash function
		FRandomStream HashStream;	
		
		HashStream.Initialize(X);
		const float XRand = HashStream.FRand();
		
		HashStream.Initialize(Y);
		const float YRand = HashStream.FRand();
		
		const int32 RandomNumber = static_cast<int32>(static_cast<float>(RAND_MAX) * XRand / (YRand + 0.01));
		const int32 Idx = FMath::Clamp(RandomNumber % PrecomputedTiles.Num(), 0, PrecomputedTiles.Num() - 1);
		return PrecomputedTiles[Idx].Get();
	}

	return nullptr;
}

void UProceduralFoliageSpawner::Simulate(int32 NumSteps)
{
	RandomStream.Initialize(RandomSeed);
	CreateProceduralFoliageInstances();

	LastCancel.Increment();

	PrecomputedTiles.Empty();
	TArray<TFuture< UProceduralFoliageTile* >> Futures;

	for (int i = 0; i < NumUniqueTiles; ++i)
	{
		UProceduralFoliageTile* NewTile = NewObject<UProceduralFoliageTile>(this);
		const int32 RandomNumber = GetRandomNumber();
		const int32 LastCancelInit = LastCancel.GetValue();

		Futures.Add(Async(EAsyncExecution::ThreadPool, [this, NewTile, RandomNumber, NumSteps, LastCancelInit]()
		{
			NewTile->Simulate(this, RandomNumber, NumSteps, LastCancelInit);
			return NewTile;
		}));
	}

	const FText StatusMessage = LOCTEXT("SimulateProceduralFoliage", "Simulate ProceduralFoliage...");
	GWarn->BeginSlowTask(StatusMessage, true, true);
	
	const int32 TotalTasks = Futures.Num();

	bool bCancelled = false;

	for (int32 FutureIdx = 0; FutureIdx < Futures.Num(); ++FutureIdx)
	{
		// Sleep for 100ms if not ready. Needed so cancel is responsive.
		while (Futures[FutureIdx].WaitFor(FTimespan::FromMilliseconds(100.0)) == false)
		{
			GWarn->StatusUpdate(FutureIdx, TotalTasks, LOCTEXT("SimulateProceduralFoliage", "Simulate ProceduralFoliage..."));

			if (GWarn->ReceivedUserCancel() && bCancelled == false)
			{
				// Increment the thread-safe counter. Tiles compare against the original count and cancel if different.
				LastCancel.Increment();
				bCancelled = true;
			}
		}

		// Even if canceled, block until all threads have exited safely. This ensures memory isn't GC'd.
		PrecomputedTiles.Add(Futures[FutureIdx].Get());		
	}

	GWarn->EndSlowTask();

	if (bCancelled)
	{
		PrecomputedTiles.Empty();
	}
	else
	{
		SetClean();
	}
}

int32 UProceduralFoliageSpawner::GetRandomNumber()
{
	return static_cast<int32>(RandomStream.FRand() * float(RAND_MAX));
}

#undef LOCTEXT_NAMESPACE

