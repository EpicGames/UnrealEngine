// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralFoliageTile.h"
#include "Engine/EngineTypes.h"
#include "CollisionQueryParams.h"
#include "ProceduralFoliageSpawner.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProceduralFoliageTile)

#define LOCTEXT_NAMESPACE "ProceduralFoliage"

UProceduralFoliageTile::UProceduralFoliageTile(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}


bool UProceduralFoliageTile::HandleOverlaps(FProceduralFoliageInstance* Instance)
{
	// If the instance survives we mark all dominated overlaps as pending removal. They will be removed from the broadphase and will not spread seeds or age.
	// Note that this introduces potential indeterminism! If the iteration order changes we could get different results. This is needed because it gives us huge performance savings.
	// Note that if the underlying data structures stay the same (i.e. no core engine changes) this should not matter. This gives us short term determinism, but not long term.

	bool bSurvived = true;
	TArray<FProceduralFoliageOverlap> Overlaps;
	Broadphase.GetOverlaps(Instance, Overlaps);

	// Check if the instance survives
	for (const FProceduralFoliageOverlap& Overlap : Overlaps)
	{
		FProceduralFoliageInstance* Dominated = FProceduralFoliageInstance::Domination(Overlap.A, Overlap.B, Overlap.OverlapType);
		if (Dominated == Instance)
		{
			bSurvived = false;
			break;
		}
	}

	if (bSurvived)
	{
		for (const FProceduralFoliageOverlap& Overlap : Overlaps)
		{
			if (FProceduralFoliageInstance* Dominated = FProceduralFoliageInstance::Domination(Overlap.A, Overlap.B, Overlap.OverlapType))
			{
				// Should only be here if we didn't survive
				check(Dominated != Instance);

				//We can't immediately remove because we're potentially iterating over existing instances.
				MarkPendingRemoval(Dominated);
			}
		}
	}
	else
	{
		//didn't survive so just die
		MarkPendingRemoval(Instance);
	}

	return bSurvived;
}

FProceduralFoliageInstance* UProceduralFoliageTile::NewSeed(const FVector& Location, float Scale, const UFoliageType* Type, float InAge, bool bBlocker)
{
	const float InitRadius = Type->GetMaxRadius() * Scale;

	FProceduralFoliageInstance* NewInst = new FProceduralFoliageInstance();
	NewInst->Location = Location;

	// make a new local random stream to avoid changes to instance randomness changing the position of all other procedural instances
	FRandomStream LocalStream = RandomStream;
	RandomStream.GetUnsignedInt(); // advance the parent stream by one

	FRotator Rotation = {0,0,0};
	Rotation.Yaw   = LocalStream.FRandRange(0, Type->RandomYaw ? 360 : 0);
	Rotation.Pitch = LocalStream.FRandRange(0, Type->RandomPitchAngle);
	NewInst->Rotation = FQuat(Rotation);
	NewInst->Age = InAge;
	NewInst->Type = Type;
	NewInst->Normal = FVector(0, 0, 1);
	NewInst->Scale = Scale;
	NewInst->bBlocker = bBlocker;

	// Don't add the seed if outside the quadtree TreeBox...
	bool bSucceedsAgainstAABBCheck = Broadphase.TestAgainstAABB(NewInst);
	if (bSucceedsAgainstAABBCheck)
	{
		// Add the seed if possible
		Broadphase.Insert(NewInst);
		const bool bSurvived = HandleOverlaps(NewInst);
		return bSurvived ? NewInst : nullptr;
	}
	else
	{
		return nullptr;
	}
}

float GetSeedMinDistance(const FProceduralFoliageInstance* Instance, const float NewInstanceAge, const int32 SimulationStep)
{
	const UFoliageType* Type = Instance->Type;
	const int32 StepsLeft = static_cast<int32>(Type->MaxAge - SimulationStep);
	const float InstanceMaxAge = Type->GetNextAge(Instance->Age, StepsLeft);
	const float NewInstanceMaxAge = Type->GetNextAge(NewInstanceAge, StepsLeft);

	const float InstanceMaxScale = Type->GetScaleForAge(InstanceMaxAge);
	const float NewInstanceMaxScale = Type->GetScaleForAge(NewInstanceMaxAge);

	const float InstanceMaxRadius = InstanceMaxScale * Type->GetMaxRadius();
	const float NewInstanceMaxRadius = NewInstanceMaxScale * Type->GetMaxRadius();

	return InstanceMaxRadius + NewInstanceMaxRadius;
}

/** Generates a random number with a normal distribution with mean=0 and variance = 1. Uses Box-Muller transformation http://mathworld.wolfram.com/Box-MullerTransformation.html */
float UProceduralFoliageTile::GetRandomGaussian()
{
	const float Rand1 = FMath::Max<float>(RandomStream.FRand(), SMALL_NUMBER);
	const float Rand2 = FMath::Max<float>(RandomStream.FRand(), SMALL_NUMBER);
	const float SqrtLn = FMath::Sqrt(-2.f * FMath::Loge(Rand1));
	const float Rand2TwoPi = Rand2 * 2.f * PI;
	const float Z1 = SqrtLn * FMath::Cos(Rand2TwoPi);
	return Z1;
}

FVector UProceduralFoliageTile::GetSeedOffset(const UFoliageType* Type, float MinDistance)
{
	//We want 10% of seeds to be the max distance so we use a z score of +- 1.64
	const float MaxZScore = 1.64f;
	const float Z1 = GetRandomGaussian();
	const float Z1Clamped = FMath::Clamp(Z1, -MaxZScore, MaxZScore);
	const float VariationDistance = Z1Clamped * Type->SpreadVariance / MaxZScore;
	const float AverageDistance = MinDistance + Type->AverageSpreadDistance;
	
	const float RandRad = FMath::Max<float>(RandomStream.FRand(), SMALL_NUMBER) * PI * 2.f;
	const FVector Dir = FVector(FMath::Cos(RandRad), FMath::Sin(RandRad), 0);
	return Dir * (AverageDistance + VariationDistance);
}

void UProceduralFoliageTile::AgeSeeds()
{
	TArray<FProceduralFoliageInstance*> NewSeeds;
	for (FProceduralFoliageInstance* Instance : InstancesSet)
	{
		if (UserCancelled()){ return; }
		if (Instance->IsAlive())
		{
			const UFoliageType* Type = Instance->Type;
			if (SimulationStep <= Type->NumSteps && Type->GetSpawnsInShade() == bSimulateOnlyInShade)
			{
				const float CurrentAge = Instance->Age;
				const float NewAge = Type->GetNextAge(Instance->Age, 1);
				const float NewScale = Type->GetScaleForAge(NewAge);

				const FVector Location = Instance->Location;

				// Replace the current instance with the newly aged version
				MarkPendingRemoval(Instance);
				if (FProceduralFoliageInstance* Inst = NewSeed(Location, NewScale, Type, NewAge))
				{
					NewSeeds.Add(Inst);
				}
			}
		}
	}

	// Save all the newly created aged instances
	for (FProceduralFoliageInstance* Seed : NewSeeds)
	{
		InstancesSet.Add(Seed);
	}

	// Get rid of the old younger versions
	FlushPendingRemovals();
}

void UProceduralFoliageTile::SpreadSeeds(TArray<FProceduralFoliageInstance*>& NewSeeds)
{
	for (FProceduralFoliageInstance* Inst : InstancesSet)
	{
		if (UserCancelled()){ return; }
		if (Inst->IsAlive() == false)
		{
			// The instance has been killed so don't bother spreading seeds. 
			// Note this introduces potential indeterminism if the order of instance traversal changes (implementation details of TSet for example)
			continue;
		}

		const UFoliageType* Type = Inst->Type;

		if (SimulationStep <= Type->NumSteps  && Type->GetSpawnsInShade() == bSimulateOnlyInShade)
		{
			for (int32 i = 0; i < Type->SeedsPerStep; ++i)
			{
				//spread new seeds
				const float NewAge = Type->GetInitAge(RandomStream);
				const float NewScale = Type->GetScaleForAge(NewAge);
				const float MinDistanceToClear = GetSeedMinDistance(Inst, NewAge, SimulationStep);
				const FVector GlobalOffset = GetSeedOffset(Type, MinDistanceToClear);
				
				if (GlobalOffset.SizeSquared2D() + SMALL_NUMBER > MinDistanceToClear*MinDistanceToClear)
				{
					const FVector NewLocation = GlobalOffset + Inst->Location;
					if (FProceduralFoliageInstance* NewInstance = NewSeed(NewLocation, NewScale, Type, NewAge))
					{
						NewSeeds.Add(NewInstance);
					}
				}
			}
		}
	}
}

void UProceduralFoliageTile::AddRandomSeeds(TArray<FProceduralFoliageInstance*>& OutInstances)
{
	const float SizeTenM2 = ( FoliageSpawner->TileSize * FoliageSpawner->TileSize ) / ( 1000.f * 1000.f );

	TMap<int32,float> MaxShadeRadii;
	TMap<int32, float> MaxCollisionRadii;
	TMap<const UFoliageType*, int32> SeedsLeftMap;
	TMap<const UFoliageType*, FRandomStream> RandomStreamPerType;

	TArray<const UFoliageType*> TypesToSeed;

	for (const FFoliageTypeObject& FoliageTypeObject : FoliageSpawner->GetFoliageTypes())
	{
		if (UserCancelled()){ return; }
		const UFoliageType* TypeInstance = FoliageTypeObject.GetInstance();
		if (TypeInstance && TypeInstance->GetSpawnsInShade() == bSimulateOnlyInShade)
		{
			{	//compute the number of initial seeds
				const int32 NumSeeds = FMath::RoundToInt(TypeInstance->GetSeedDensitySquared() * SizeTenM2);
				SeedsLeftMap.Add(TypeInstance, NumSeeds);
				if (NumSeeds > 0)
				{
					TypesToSeed.Add(TypeInstance);
				}
			}

			{	//save the random stream per type
				RandomStreamPerType.Add(TypeInstance, FRandomStream(TypeInstance->DistributionSeed + FoliageSpawner->RandomSeed + RandomSeed));
			}

			{	//compute the needed offsets for initial seed variance
				const int32 DistributionSeed = TypeInstance->DistributionSeed;
				const float MaxScale = TypeInstance->GetScaleForAge(TypeInstance->MaxAge);
				const float TypeMaxCollisionRadius = MaxScale * TypeInstance->CollisionRadius;
				if (float* MaxRadius = MaxCollisionRadii.Find(DistributionSeed))
				{
					*MaxRadius = FMath::Max(*MaxRadius, TypeMaxCollisionRadius);
				}
				else
				{
					MaxCollisionRadii.Add(DistributionSeed, TypeMaxCollisionRadius);
				}

				const float TypeMaxShadeRadius = MaxScale * TypeInstance->ShadeRadius;
				if (float* MaxRadius = MaxShadeRadii.Find(DistributionSeed))
				{
					*MaxRadius = FMath::Max(*MaxRadius, TypeMaxShadeRadius);
				}
				else
				{
					MaxShadeRadii.Add(DistributionSeed, TypeMaxShadeRadius);
				}
			}
			
		}
	}

	int32 TypeIdx = -1;
	const int32 NumTypes = TypesToSeed.Num();
	int32 TypesLeftToSeed = NumTypes;
	const int32 LastShadeCastingIndex = InstancesArray.Num() - 1; //when placing shade growth types we want to spawn in shade if possible
	while (TypesLeftToSeed > 0)
	{
		if (UserCancelled()){ return; }
		TypeIdx = (TypeIdx + 1) % NumTypes;	//keep cycling through the types that we spawn initial seeds for to make sure everyone gets fair chance

		if (const UFoliageType* Type = TypesToSeed[TypeIdx])
		{
			int32& SeedsLeft = SeedsLeftMap.FindChecked(Type);
			if (SeedsLeft == 0)
			{
				continue;
			}

			const float NewAge = Type->GetInitAge(RandomStream);
			const float Scale = Type->GetScaleForAge(NewAge);

			FRandomStream& TypeRandomStream = RandomStreamPerType.FindChecked(Type);
			FVector::FReal InitX = 0.f;
			FVector::FReal InitY = 0.f;
			float NeededRadius = 0.f;

			if (bSimulateOnlyInShade && LastShadeCastingIndex >= 0)
			{
				const int32 InstanceSpawnerIdx = TypeRandomStream.RandRange(0, LastShadeCastingIndex);
				const FProceduralFoliageInstance& Spawner = InstancesArray[InstanceSpawnerIdx];
				InitX = Spawner.Location.X;
				InitY = Spawner.Location.Y;
				NeededRadius = Spawner.GetCollisionRadius() * (Scale + Spawner.Type->GetScaleForAge(Spawner.Age));
			}
			else
			{
				InitX = TypeRandomStream.FRandRange(0, FoliageSpawner->TileSize);
				InitY = TypeRandomStream.FRandRange(0, FoliageSpawner->TileSize);
				NeededRadius = MaxShadeRadii.FindRef(Type->DistributionSeed);
			}

			const FVector::FReal Rad = RandomStream.FRandRange(0, PI*2.f);
			
			
			const FVector GlobalOffset = (RandomStream.FRandRange(0, Type->MaxInitialSeedOffset) + NeededRadius) * FVector(FMath::Cos(Rad), FMath::Sin(Rad), 0.f);

			const FVector::FReal X = InitX + GlobalOffset.X;
			const FVector::FReal Y = InitY + GlobalOffset.Y;

			if (FProceduralFoliageInstance* NewInst = NewSeed(FVector(X, Y, 0.f), Scale, Type, NewAge))
			{
				OutInstances.Add(NewInst);
			}

			--SeedsLeft;
			if (SeedsLeft == 0)
			{
				--TypesLeftToSeed;
			}
		}
	}
}

void UProceduralFoliageTile::MarkPendingRemoval(FProceduralFoliageInstance* ToRemove)
{
	if (ToRemove->IsAlive())
	{
		Broadphase.Remove(ToRemove);	//we can remove from broadphase right away
		ToRemove->TerminateInstance();
		PendingRemovals.Add(ToRemove);
	}
}

void UProceduralFoliageTile::RemoveInstances()
{
	for (FProceduralFoliageInstance* Inst : InstancesSet)
	{
		MarkPendingRemoval(Inst);
	}

	InstancesArray.Empty();
	FlushPendingRemovals();
}

void UProceduralFoliageTile::InstancesToArray()
{
	InstancesArray.Empty(InstancesSet.Num());
	for (FProceduralFoliageInstance* FromInst : InstancesSet)
	{
		// Blockers do not get instantiated so don't bother putting it into array
		if (FromInst->bBlocker == false)
		{
			new(InstancesArray)FProceduralFoliageInstance(*FromInst);
		}
	}
}

void UProceduralFoliageTile::RemoveInstance(FProceduralFoliageInstance* ToRemove)
{
	if (ToRemove->IsAlive())
	{
		Broadphase.Remove(ToRemove);
		ToRemove->TerminateInstance();
	}
	
	InstancesSet.Remove(ToRemove);
	delete ToRemove;
}

void UProceduralFoliageTile::FlushPendingRemovals()
{
	for (FProceduralFoliageInstance* ToRemove : PendingRemovals)
	{
		RemoveInstance(ToRemove);
	}

	PendingRemovals.Empty();
}

void UProceduralFoliageTile::InitSimulation(const UProceduralFoliageSpawner* InFoliageSpawner, const int32 InRandomSeed)
{
	RandomSeed = InRandomSeed;
	RandomStream.Initialize(RandomSeed);
	FoliageSpawner = InFoliageSpawner;
	SimulationStep = 0;
	Broadphase = FProceduralFoliageBroadphase(FoliageSpawner->TileSize, FoliageSpawner->MinimumQuadTreeSize);
}

bool UProceduralFoliageTile::UserCancelled() const
{
	return FoliageSpawner->LastCancel.GetValue() != LastCancel;
}


void UProceduralFoliageTile::StepSimulation()
{
	if (UserCancelled()){ return; }
	TArray<FProceduralFoliageInstance*> NewInstances;
	if (SimulationStep == 0)
	{
		AddRandomSeeds(NewInstances);
	}
	else
	{
		AgeSeeds();
		SpreadSeeds(NewInstances);
	}

	InstancesSet.Append(NewInstances);

	FlushPendingRemovals();
}

void UProceduralFoliageTile::RunSimulation(const int32 MaxNumSteps, bool bOnlyInShade)
{
	int32 MaxSteps = 0;

	for (const FFoliageTypeObject& FoliageTypeObject : FoliageSpawner->GetFoliageTypes())
	{
		const UFoliageType* TypeInstance = FoliageTypeObject.GetInstance();
		if (TypeInstance && TypeInstance->GetSpawnsInShade() == bOnlyInShade)
		{
			MaxSteps = FMath::Max(MaxSteps, TypeInstance->NumSteps + 1);
		}
	}

	if (MaxNumSteps >= 0)
	{
		MaxSteps = FMath::Min(MaxSteps, MaxNumSteps);	//only take as many steps as given
	}

	SimulationStep = 0;
	bSimulateOnlyInShade = bOnlyInShade;
	for (int32 Step = 0; Step < MaxSteps; ++Step)
	{
		StepSimulation();
		++SimulationStep;
	}

	InstancesToArray();
}

void UProceduralFoliageTile::Simulate(const UProceduralFoliageSpawner* InFoliageSpawner, const int32 InRandomSeed, const int32 MaxNumSteps, const int32 InLastCancel)
{
	LastCancel = InLastCancel;
	InitSimulation(InFoliageSpawner, InRandomSeed);

	RunSimulation(MaxNumSteps, false);
	RunSimulation(MaxNumSteps, true);
}


void UProceduralFoliageTile::BeginDestroy()
{
	Super::BeginDestroy();
	RemoveInstances();
}

void UProceduralFoliageTile::ExtractDesiredInstances(TArray<FDesiredFoliageInstance>& OutInstances, const FTransform& WorldTM, const FVector2D& ActorVolumeLocation, FVector::FReal ActorVolumeMaxExtent, const FGuid& ProceduralGuid, const FVector::FReal HalfHeight, const FBodyInstance* VolumeBodyInstance, bool bEmptyTileInfo)
{
	InstancesToArray();

	const FCollisionQueryParams Params(NAME_None, FCollisionQueryParams::GetUnknownStatId(), true);
	FHitResult Hit;

	OutInstances.Reserve(InstancesSet.Num());
	for (const FProceduralFoliageInstance& Instance : InstancesArray)
	{
		FVector StartRay = Instance.Location + WorldTM.GetLocation();
		StartRay.Z += HalfHeight;
		FVector EndRay = StartRay;
		EndRay.Z -= (HalfHeight*2.f + 10.f);	//add 10cm to bottom position of raycast. This is needed because volume is usually placed directly on geometry and then you get precision issues

		// Apply FoliageType's Falloff
		FVector2D Position(StartRay);
		if (Instance.Type->DensityFalloff.IsInstanceFiltered(Position, ActorVolumeLocation, ActorVolumeMaxExtent))
		{
			continue;
		}
		
		FDesiredFoliageInstance* DesiredInst = new (OutInstances)FDesiredFoliageInstance(StartRay, EndRay, Instance.Type, Instance.GetMaxRadius());
		DesiredInst->Rotation = Instance.Rotation;
		DesiredInst->ProceduralGuid = ProceduralGuid;
		DesiredInst->Age = Instance.Age;
		DesiredInst->ProceduralVolumeBodyInstance = VolumeBodyInstance;
		DesiredInst->PlacementMode = EFoliagePlacementMode::Procedural;
	}

	if (bEmptyTileInfo)
	{
		Empty();
	}
}

void UProceduralFoliageTile::CopyInstancesToTile(UProceduralFoliageTile* ToTile, const FBox2D& LocalAABB, const FTransform& RelativeTM, const float Overlap) const
{
	//@todo proc foliage: Would be better to use the max radius of any instances in the tile instead of overlap to define the outer AABB

	TArray<FProceduralFoliageInstance*> InstancesIncludingOverlap;
	const FBox2D OuterLocalAABB(LocalAABB.Min, LocalAABB.Max + Overlap);
	
	// Get all the instances in the outer AABB (so we include potential blockers)
	GetInstancesInAABB(OuterLocalAABB, InstancesIncludingOverlap);


	ToTile->AddInstances(InstancesIncludingOverlap, RelativeTM, LocalAABB);
}

void UProceduralFoliageTile::Empty()
{
	Broadphase.Empty();
	InstancesArray.Empty();
	
	for (FProceduralFoliageInstance* Inst : InstancesSet)
	{
		delete Inst;
	}

	InstancesSet.Empty();
	PendingRemovals.Empty();
}

void UProceduralFoliageTile::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	for (FProceduralFoliageInstance* Inst : InstancesSet)
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(FProceduralFoliageInstance));
	}
	
	//@TODO: account for broadphase
}


void UProceduralFoliageTile::GetInstancesInAABB(const FBox2D& LocalAABB, TArray<FProceduralFoliageInstance*>& OutInstances, bool bFullyContainedOnly) const
{
	TArray<FProceduralFoliageInstance*> InstancesInAABB;
	Broadphase.GetInstancesInBox(LocalAABB, InstancesInAABB);

	OutInstances.Reserve(OutInstances.Num() + InstancesInAABB.Num());
	for (FProceduralFoliageInstance* Inst : InstancesInAABB)
	{
		const float Rad = Inst->GetMaxRadius();
		const FVector& Location = Inst->Location;

		if (!bFullyContainedOnly || (Location.X - Rad >= LocalAABB.Min.X && Location.X + Rad <= LocalAABB.Max.X && Location.Y - Rad >= LocalAABB.Min.Y && Location.Y + Rad <= LocalAABB.Max.Y))
		{
			OutInstances.Add(Inst);
		}
	}

	// Sort the instances by location.
	// This protects us from any future modifications made to the broadphase that would impact the order in which instances are located in the AABB.
	OutInstances.Sort([](const FProceduralFoliageInstance& A, const FProceduralFoliageInstance& B)
	{
		return (B.Location.X == A.Location.X) ? (B.Location.Y > A.Location.Y) : (B.Location.X > A.Location.X);
	});
}

void UProceduralFoliageTile::AddInstances(const TArray<FProceduralFoliageInstance*>& NewInstances, const FTransform& RelativeTM, const FBox2D& InnerLocalAABB)
{
	for (const FProceduralFoliageInstance* Inst : NewInstances)
	{
		// We need the local space because we're comparing it to the AABB
		const FVector& Location = Inst->Location;	
		const float Radius = Inst->GetMaxRadius();
		
		// Instances in InnerLocalAABB or on the border of the max sides of the AABB will be visible and instantiated by this tile
		// Instances outside of the InnerLocalAABB are only used for rejection purposes. This is needed for overlapping tiles
		// The actual instantiation of the object will be taken care of by a different tile
		const bool bIsOutsideInnerLocalAABB = Location.X + Radius <= InnerLocalAABB.Min.X
											|| Location.X - Radius > InnerLocalAABB.Max.X
											|| Location.Y + Radius <= InnerLocalAABB.Min.Y
											|| Location.Y - Radius > InnerLocalAABB.Max.Y;

		const FVector NewLocation = RelativeTM.TransformPosition(Inst->Location);
		if (FProceduralFoliageInstance* NewInst = NewSeed(NewLocation, Inst->Scale, Inst->Type, Inst->Age, bIsOutsideInnerLocalAABB))
		{			
			InstancesSet.Add(NewInst);
		}
	}

	FlushPendingRemovals();
}

#undef LOCTEXT_NAMESPACE

