// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralFoliageVolume.h"
#include "Components/BrushComponent.h"
#include "Engine/World.h"
#include "ProceduralFoliageComponent.h"
#include "ProceduralFoliageSpawner.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProceduralFoliageVolume)

AProceduralFoliageVolume::AProceduralFoliageVolume(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	ProceduralComponent = ObjectInitializer.CreateDefaultSubobject<UProceduralFoliageComponent>(this, TEXT("ProceduralFoliageComponent"));
	ProceduralComponent->SetSpawningVolume(this);

	if (UBrushComponent* MyBrushComponent = GetBrushComponent())
	{
		MyBrushComponent->SetCollisionObjectType(ECC_WorldStatic);
		MyBrushComponent->SetCollisionResponseToAllChannels(ECR_Ignore);

		// This is important because the volume overlaps with all procedural foliage
		// That means during streaming we'll get a huge hitch for UpdateOverlaps
		MyBrushComponent->SetGenerateOverlapEvents(false);
	}
}

#if WITH_EDITOR
void AProceduralFoliageVolume::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	if (!GetWorld()->IsGameWorld() && GetWorld()->IsPartitionedWorld() && !WorldPartitionActorLoader)
	{
		WorldPartitionActorLoader = new FLoaderAdapterActor(this);
	}
}

void AProceduralFoliageVolume::BeginDestroy()
{
	if (WorldPartitionActorLoader)
	{
		delete WorldPartitionActorLoader;
		WorldPartitionActorLoader = nullptr;
	}

	Super::BeginDestroy();
}

IWorldPartitionActorLoaderInterface::ILoaderAdapter* AProceduralFoliageVolume::GetLoaderAdapter()
{
	return WorldPartitionActorLoader;
}

void AProceduralFoliageVolume::PostEditImport()
{
	// Make sure that this is the component's spawning volume
	ProceduralComponent->SetSpawningVolume(this);
}

bool AProceduralFoliageVolume::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (ProceduralComponent && ProceduralComponent->FoliageSpawner)
	{
		Objects.Add(ProceduralComponent->FoliageSpawner);
	}
	return true;
}
#endif

