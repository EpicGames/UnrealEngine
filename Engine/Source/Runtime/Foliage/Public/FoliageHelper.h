// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"

/**
 * Foliage Helper class
 */
class FFoliageHelper
{
public:
#if WITH_EDITOR
	static inline FName GetFoliageActorInstanceTag()
	{
		static FName NAME_FoliageActorInstanceTag("FoliageActorInstance");
		return NAME_FoliageActorInstanceTag;
	}

	static inline void SetIsOwnedByFoliage(AActor* InActor, bool bOwned = true) 
	{ 
		if (InActor) 
		{ 
			FSetActorHiddenInSceneOutliner SetHidden(InActor, bOwned);
			if (bOwned)
			{
				InActor->Tags.AddUnique(GetFoliageActorInstanceTag());
			}
			else
			{
				InActor->Tags.Remove(GetFoliageActorInstanceTag());
			}
		} 
	}

	static inline bool IsOwnedByFoliage(const AActor* InActor)
	{
		return InActor && InActor->ActorHasTag(GetFoliageActorInstanceTag());
	}
#endif
};