// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/SoftObjectPtr.h"

class AInstancedFoliageActor;
class UActorComponent;

typedef int32 FFoliageInstanceBaseId;
typedef TSoftObjectPtr<UActorComponent> FFoliageInstanceBasePtr;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
typedef TLazyObjectPtr<UActorComponent> FFoliageInstanceBasePtr_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITORONLY_DATA
/**
 * FFoliageInstanceBaseInfo
 * Cached transform information about base component
 * Used for adjusting instance transforms after operations on base components with foliage painted on them.
 */
struct FFoliageInstanceBaseInfo
{
	// tors
	FFoliageInstanceBaseInfo();
	FFoliageInstanceBaseInfo(UActorComponent* InComponent);

	// Cache the location and rotation from the actor
	void UpdateLocationFromComponent(UActorComponent* InComponent);

	// Serializer
	friend FArchive& operator<<(FArchive& Ar, FFoliageInstanceBaseInfo& ComponentInfo);

	FFoliageInstanceBasePtr BasePtr;
	FVector		CachedLocation;
	FRotator	CachedRotation;
	FVector		CachedDrawScale;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Deprecated data stored for deserializaton done in PostLoad
	FFoliageInstanceBasePtr_DEPRECATED BasePtr_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

struct FFoliageInstanceBaseCache
{
	FFoliageInstanceBaseCache();

	/* Adds new base to cache and/or returns existing base Id*/
	FOLIAGE_API FFoliageInstanceBaseId AddInstanceBaseId(UActorComponent* InComponent);
	
	/* Returns Id for a registered base component, invalid Id otherwise */
	FOLIAGE_API FFoliageInstanceBaseId GetInstanceBaseId(UActorComponent* InComponent) const;
	
	/* Returns Id for a registered base component, invalid Id otherwise */
	FOLIAGE_API FFoliageInstanceBaseId GetInstanceBaseId(const FFoliageInstanceBasePtr& BasePtr) const;
	
	/* Returns registered base component for specified Id, null otherwise */
	FOLIAGE_API FFoliageInstanceBasePtr GetInstanceBasePtr(FFoliageInstanceBaseId BaseId) const;
	
	/* Returns base info for registered Id */
	FOLIAGE_API FFoliageInstanceBaseInfo GetInstanceBaseInfo(FFoliageInstanceBaseId BaseId) const;
	
	/* Updates base info for a specified base component and returns updated info */
	FOLIAGE_API FFoliageInstanceBaseInfo UpdateInstanceBaseInfoTransform(UActorComponent* InComponent);

	/* Compacts cross-level references, removing dead links */
	FOLIAGE_API static void CompactInstanceBaseCache(AInstancedFoliageActor* IFA);
	
	/* Refreshes base component transforms cache */
	FOLIAGE_API void UpdateInstanceBaseCachedTransforms();
	
	friend FArchive& operator<<(FArchive& Ar, FFoliageInstanceBaseCache& InstanceBaseCache);
		
	//
	FOLIAGE_API static FFoliageInstanceBaseId InvalidBaseId;

	// ID generator
	// serialized
	FFoliageInstanceBaseId NextBaseId;

	// Map for looking up base info by ID
	// serialized
	TMap<FFoliageInstanceBaseId, FFoliageInstanceBaseInfo> InstanceBaseMap;
	
	// Map for looking up base ID by pointer to component
	// transient
	TMap<FFoliageInstanceBasePtr, FFoliageInstanceBaseId> InstanceBaseInvMap;
	
	// Map for detecting removed bases
	// serialized
	TMap<TSoftObjectPtr<UWorld>, TArray<FFoliageInstanceBasePtr>> InstanceBaseLevelMap;
	
	// Deprecated data stored for deserializaton done in PostLoad
	TMap<TSoftObjectPtr<UWorld>, TArray<FFoliageInstanceBasePtr_DEPRECATED>> InstanceBaseLevelMap_DEPRECATED;
};

#endif// WITH_EDITORONLY_DATA
