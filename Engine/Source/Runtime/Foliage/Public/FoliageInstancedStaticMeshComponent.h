// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Misc/Guid.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "FoliageInstancedStaticMeshComponent.generated.h"

class AController;

/**
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_SevenParams(FInstancePointDamageSignature, int32, InstanceIndex, float, Damage, class AController*, InstigatedBy, FVector, HitLocation, FVector, ShotFromDirection, const class UDamageType*, DamageType, AActor*, DamageCauser);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_SevenParams(FInstanceRadialDamageSignature, const TArray<int32>&, Instances, const TArray<float>&, Damages, class AController*, InstigatedBy, FVector, Origin, float, MaxRadius, const class UDamageType*, DamageType, AActor*, DamageCauser);

UCLASS(ClassGroup = Foliage, Blueprintable, MinimalAPI)
class UFoliageInstancedStaticMeshComponent : public UHierarchicalInstancedStaticMeshComponent
{
	GENERATED_UCLASS_BODY()
public:

	UPROPERTY(BlueprintAssignable, Category = "Game|Damage")
	FInstancePointDamageSignature OnInstanceTakePointDamage;

	UPROPERTY(BlueprintAssignable, Category = "Game|Damage")
	FInstanceRadialDamageSignature OnInstanceTakeRadialDamage;

	UPROPERTY()
	bool bEnableDiscardOnLoad;

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	uint64 FoliageHiddenEditorViews;
#endif// WITH_EDITOR

	FOLIAGE_API virtual void ReceiveComponentDamage(float DamageAmount, FDamageEvent const& DamageEvent, AController* EventInstigator, AActor* DamageCauser) override;

	//USceneComponent Interface
#if WITH_EDITOR
	FOLIAGE_API virtual uint64 GetHiddenEditorViews() const override;
	FOLIAGE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
#endif// WITH_EDITOR

	/** Used by procedural generation to link generated component with its creator */
	void SetGenerationGuid(const FGuid& InGuid) { GenerationGuid = InGuid; }
	const FGuid& GetGenerationGuid() const { return GenerationGuid; }

private:

	UPROPERTY()
	FGuid GenerationGuid;
};

