// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SubclassOf.h"

class ULevel;
class UActorComponent;

/**
 * Foliage Edit mode module interface
 */
class IFoliageEditModuleBase : public IModuleInterface
{
public:
#if WITH_EDITOR
	/** Move the selected foliage to the specified level */
	virtual void MoveSelectedFoliageToLevel(ULevel* InTargetLevel) = 0;
	/** Notifies us that the foliage has been externally changed and needs refreshing.  */
	virtual void UpdateMeshList() = 0;
	virtual bool CanMoveSelectedFoliageToLevel(ULevel* InTargetLevel) const = 0;

	virtual bool ShouldIgnoreComponentForBaseID(UActorComponent* InComponent) const = 0;
	virtual void RegisterComponentBaseIDClassToIgnore(TSubclassOf<UActorComponent> InClassToIgnore) = 0;
	virtual void UnregisterComponentBaseIDClassToIgnore(TSubclassOf<UActorComponent> InClassToUnregister) = 0;

	FOLIAGE_API static IFoliageEditModuleBase* Get();
#endif
	
protected:

#if WITH_EDITOR
	FOLIAGE_API static void SetFoliageEditModulePtr(IFoliageEditModuleBase* InFoliageEditModulePtr);
#endif

private:
#if WITH_EDITOR
	static IFoliageEditModuleBase* FoliageEditModulePtr;
#endif
};

class IFoliageModule : public IModuleInterface
{
public:
};
