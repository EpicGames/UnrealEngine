// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "FoliageStatistics.generated.h"

class UStaticMesh;

UCLASS(MinimalAPI)
class UFoliageStatistics : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/**
	* Counts how many foliage instances overlap a given sphere
	*
	* @param	Mesh			The static mesh we are interested in counting
	* @param	CenterPosition	The center position of the sphere
	* @param	Radius			The radius of the sphere.
	*
	* return number of foliage instances with their mesh set to Mesh that overlap the sphere
	*/
	UFUNCTION(BlueprintCallable, Category = "Foliage", meta = (WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static FOLIAGE_API int32 FoliageOverlappingSphereCount(UObject* WorldContextObject, const UStaticMesh* StaticMesh, FVector CenterPosition, float Radius);

	/** 
	 *	Gets the number of instances overlapping a provided box
	 *	@param StaticMesh Mesh to count
	 *	@param Box Box to overlap
	 */
	UFUNCTION(BlueprintCallable, Category = "Foliage", meta = (WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static FOLIAGE_API int32 FoliageOverlappingBoxCount(UObject* WorldContextObject, const UStaticMesh* StaticMesh, FBox Box);

	/** 
	 *  Get the transform of every instance overlapping the provided FBox
	 *	@param StaticMesh Mesh to get instances of
	 *	@param Box Box to use for overlap
	 *	@param OutTransforms Array to populate with transforms
	 */
	UFUNCTION(BlueprintCallable, Category = "Foliage", meta = (WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static FOLIAGE_API void FoliageOverlappingBoxTransforms(UObject* WorldContextObject, const UStaticMesh* StaticMesh, FBox Box, TArray<FTransform>& OutTransforms);

	/** 
	 *  DEBUG FUNCTION: This is not fast, use only for debug/development.
	 *  Gets an instance count for each unique mesh type overlapping the given sphere.
	 *  @param	CenterPosition	The center position of the sphere
	 *  @param	Radius			The radius of the sphere.
	 *	@param OutMeshCounts	Map of Meshes to instance counts
	 */
	static FOLIAGE_API void FoliageOverlappingMeshCounts_Debug(UObject* WorldContextObject, FVector CenterPosition, float Radius, TMap<UStaticMesh*, int32>& OutMeshCounts);
};

