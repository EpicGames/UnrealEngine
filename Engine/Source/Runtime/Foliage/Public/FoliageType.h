// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Engine/EngineTypes.h"
#include "Components/PrimitiveComponent.h"
#include "Curves/CurveFloat.h"
#include "VT/RuntimeVirtualTextureEnum.h"
#include "FoliageType.generated.h"

class UStaticMesh;

UENUM()
enum FoliageVertexColorMask : int
{
	FOLIAGEVERTEXCOLORMASK_Disabled UMETA(DisplayName="Disabled"),
	FOLIAGEVERTEXCOLORMASK_Red		UMETA(DisplayName="Red"),
	FOLIAGEVERTEXCOLORMASK_Green	UMETA(DisplayName="Green"),
	FOLIAGEVERTEXCOLORMASK_Blue		UMETA(DisplayName="Blue"),
	FOLIAGEVERTEXCOLORMASK_Alpha	UMETA(DisplayName="Alpha"),
};

UENUM()
enum class EVertexColorMaskChannel : uint8
{
	Red,
	Green,
	Blue,
	Alpha,

	MAX_None	UMETA(Hidden)
};

USTRUCT()
struct FFoliageVertexColorChannelMask
{
	GENERATED_USTRUCT_BODY()

	/** 
	 *  When checked, foliage will be masked from this mesh using this color channel
	 */
	UPROPERTY(EditAnywhere, Category=VertexColorMask)
	uint32 UseMask:1;

	/** Specifies the threshold value above which the static mesh vertex color value must be, in order for foliage instances to be placed in a specific area */
	UPROPERTY(EditAnywhere, Category=VertexColorMask)
	float MaskThreshold;

	/** 
	 *  When unchecked, foliage instances will be placed only when the vertex color in the specified channel(s) is above the threshold amount. 
	 *  When checked, the vertex color must be less than the threshold amount 
	 */
	UPROPERTY(EditAnywhere, Category=VertexColorMask)
	uint32 InvertMask:1;

	FFoliageVertexColorChannelMask()
		: UseMask(false)
		, MaskThreshold(0.5f)
		, InvertMask(false)
	{}
};

UENUM()
enum class EFoliageScaling : uint8
{
	/** Foliage instances will have uniform X,Y and Z scales. */
	Uniform,
	/** Foliage instances will have random X,Y and Z scales. */
	Free,
	/** Locks the X and Y axis scale. */
	LockXY,
	/** Locks the X and Z axis scale. */
	LockXZ,
	/** Locks the Y and Z axis scale. */
	LockYZ
};

USTRUCT()
struct FFoliageDensityFalloff
{
	GENERATED_USTRUCT_BODY()

	FFoliageDensityFalloff();

	UPROPERTY(Category = Procedural, EditAnywhere, meta = (Subcategory = "Density"))
	bool bUseFalloffCurve = false;

	/**
	 * Density as a function of normalized distance (i.e. distance from Procedural Foliage Volume / Max Volume Extent).
	 * X = 0 corresponds to Normalized distance = 0, X = 1 corresponds to Normalized distance = Max distance.
	 * Y = 0 corresponds to 0% probability of keeping instance, Y = 1 corresponds to 100% probability of keeping instance.
	 */
	UPROPERTY(Category = Procedural, EditAnywhere, meta = (Subcategory = "Density", XAxisName = "Normalized Distance", YAxisName = "Density Factor"))
	FRuntimeFloatCurve FalloffCurve;

	FOLIAGE_API bool IsInstanceFiltered(const FVector2D& Position, const FVector2D& Origin, FVector::FReal MaxDistance) const;
	FOLIAGE_API float GetDensityFalloffValue(const FVector2D& Position, const FVector2D& Origin, FVector::FReal MaxDistance) const;
};

UCLASS(abstract, hidecategories = Object, editinlinenew, MinimalAPI, BlueprintType, Blueprintable)
class UFoliageType : public UObject
{
	GENERATED_UCLASS_BODY()

	/* Gets/Sets the source data associated with this FoliageType */
	virtual UObject* GetSource() const PURE_VIRTUAL(UFoliageType::GetSource, return nullptr; );
		
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;

	virtual bool IsNotAssetOrBlueprint() const;
	FOLIAGE_API FVector3f GetRandomScale() const;
	
#if WITH_EDITOR
	virtual FString GetDefaultNewAssetName() const
	{
		return TEXT("NewFoliage");
	}

	virtual void SetSource(UObject* InSource) PURE_VIRTUAL(UFoliageType::SetSource, );
	virtual void UpdateBounds() {}
	/* Lets subclasses decide if the InstancedFoliageActor should reallocate its instances if the specified property change event occurs */
	virtual bool IsFoliageReallocationRequiredForPropertyChange(const FProperty* Property) const { return true; }

	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool IsSourcePropertyChange(const FProperty* Property) const { return false; }

	/* Notifies all relevant foliage actors that HiddenEditorView mask has been changed */
	FOLIAGE_API void OnHiddenEditorViewMaskChanged(UWorld* InWorld);

	/* Nice and clean name for user interface */
	FOLIAGE_API FName GetDisplayFName() const;
#endif

	/* A GUID that is updated every time the foliage type is modified, 
	   so foliage placed in the level can detect the FoliageType has changed. */
	UPROPERTY()
	FGuid UpdateGuid;


public:
	// PAINTING

	/** Foliage instances will be placed at this density, specified in instances per 1000x1000 unit area */
	UPROPERTY(EditAnywhere, Category=Painting, meta=(DisplayName="Density / 1Kuu", UIMin = 0, ClampMin = 0, UIMax = 10000, ClampMax = 10000))
	float Density;

	/** The factor by which to adjust the density of instances. Values >1 will increase density while values <1 will decrease it. */
	UPROPERTY(EditAnywhere, Category=Painting, meta=(UIMin=0, ClampMin=0, UIMax = 1000, ClampMax = 1000, ReapplyCondition="ReapplyDensity"))
	float DensityAdjustmentFactor;

	/** The minimum distance between foliage instances */
	UPROPERTY(EditAnywhere, Category=Painting, meta=(UIMin=0, ClampMin=0, UIMax = 100000, ClampMax = 100000, ReapplyCondition="ReapplyRadius"))
	float Radius;

	/** Option to override radius used to detect collision with other instances when painting in single instance mode */
	UPROPERTY(EditAnywhere, Category = Painting)
	bool bSingleInstanceModeOverrideRadius = false;

	/** The radius used in single instance mode to detect collision with other instances */
	UPROPERTY(EditAnywhere, Category = Painting, meta = (EditCondition = "bSingleInstanceModeOverrideRadius", UIMin = 0, ClampMin = 0, UIMax = 100000, ClampMax = 100000))
	float SingleInstanceModeRadius = 0.f;

	/** Specifies foliage instance scaling behavior when painting. */
	UPROPERTY(EditAnywhere, Category=Painting, meta=(ReapplyCondition="ReapplyScaling"))
	EFoliageScaling Scaling;

	/** Specifies the range of scale, from minimum to maximum, to apply to a foliage instance's X Scale property */
	UPROPERTY(EditAnywhere, Category=Painting, meta=(ClampMin="0.001", UIMin="0.001", ReapplyCondition="ReapplyScaleX"))
	FFloatInterval ScaleX;

	/** Specifies the range of scale, from minimum to maximum, to apply to a foliage instance's Y Scale property */
	UPROPERTY(EditAnywhere, Category=Painting, meta=(ClampMin="0.001", UIMin="0.001", ReapplyCondition="ReapplyScaleY"))
	FFloatInterval ScaleY;

	/** Specifies the range of scale, from minimum to maximum, to apply to a foliage instance's Z Scale property */
	UPROPERTY(EditAnywhere, Category=Painting, meta=(ClampMin="0.001", UIMin="0.001", ReapplyCondition="ReapplyScaleZ"))
	FFloatInterval ScaleZ;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Painting, meta=(HideBehind="VertexColorMask"))
	FFoliageVertexColorChannelMask VertexColorMaskByChannel[(uint8)EVertexColorMaskChannel::MAX_None];

	/** 
	 *  When painting on static meshes, foliage instance placement can be limited to areas where the static mesh has values in the selected vertex color channel(s). 
	 *  This allows a static mesh to mask out certain areas to prevent foliage from being placed there
	 */
	UPROPERTY()
	TEnumAsByte<enum FoliageVertexColorMask> VertexColorMask_DEPRECATED;

	/** Specifies the threshold value above which the static mesh vertex color value must be, in order for foliage instances to be placed in a specific area */
	UPROPERTY()
	float VertexColorMaskThreshold_DEPRECATED;

	/** 
	 *  When unchecked, foliage instances will be placed only when the vertex color in the specified channel(s) is above the threshold amount. 
	 *  When checked, the vertex color must be less than the threshold amount 
	 */
	UPROPERTY()
	uint32 VertexColorMaskInvert_DEPRECATED:1;

public:
	// PLACEMENT

	/** Specifies a range from minimum to maximum of the offset to apply to a foliage instance's Z location */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Placement, meta=(DisplayName="Z Offset", ReapplyCondition="ReapplyZOffset"))
	FFloatInterval ZOffset;

	/** Whether foliage instances should have their angle adjusted away from vertical to match the normal of the surface they're painted on 
	 *  If AlignToNormal is enabled and RandomYaw is disabled, the instance will be rotated so that the +X axis points down-slope */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Placement, meta=(ReapplyCondition="ReapplyAlignToNormal"))
	uint32 AlignToNormal:1;

	/**	Whether the normal should be averaged on a number of samples around the hit location */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Placement, meta=(ToolTip="Will average normal based on Foliage Type base radius (this as a cost as it will do extra line traces)"))
	uint32 AverageNormal:1;

	/** Average Normal should use all hit components or only the original hit component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Placement, meta = (EditCondition = "AverageNormal", ToolTip = "Whether to discard normals originating from other hit components or not when averaging normals"))
	uint32 AverageNormalSingleComponent:1;

	/** The maximum angle in degrees that foliage instances will be adjusted away from the vertical */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Placement, meta=(UIMin = 0, ClampMin = 0, UIMax = 359, ClampMax = 359, HideBehind="AlignToNormal"))
	float AlignMaxAngle;

	/** If selected, foliage instances will have a random yaw rotation around their vertical axis applied */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Placement, meta=(ReapplyCondition="ReapplyRandomYaw"))
	uint32 RandomYaw:1;

	/** A random pitch adjustment can be applied to each instance, up to the specified angle in degrees, from the original vertical */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Placement, meta=(UIMin = 0, ClampMin = 0, UIMax = 359, ClampMax = 359, ReapplyCondition="ReapplyRandomPitchAngle"))
	float RandomPitchAngle;

	/* Foliage instances will only be placed on surfaces sloping in the specified angle range from the horizontal */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Placement, meta=(UIMin=0, ClampMin = 0, UIMax = 359, ClampMax = 359, ReapplyCondition="ReapplyGroundSlope"))
	FFloatInterval GroundSlopeAngle;

	/* The valid altitude range where foliage instances will be placed, specified using minimum and maximum world coordinate Z values */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Placement, meta=(ReapplyCondition="ReapplyHeight"))
	FFloatInterval Height;

	/** If layer names are specified, painting on landscape will limit the foliage to areas of landscape with the specified layers painted */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=Placement, meta=(ReapplyCondition="ReapplyLandscapeLayers", DisplayName="Inclusion Landscape Layers"))
	TArray<FName> LandscapeLayers;

	/** Specifies the minimum value above which the landscape layer weight value must be, in order for foliage instances to be placed in a specific area */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=Placement, meta=(UIMin=0, ClampMin = 0, UIMax = 1, ClampMax = 1, HideBehind="LandscapeLayers", DisplayName="Minimum Inclusion Landscape Weight"))
	float MinimumLayerWeight;

	/** If layer names are specified, painting on landscape will exclude the foliage to areas of landscape without the specified layers painted */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Placement, meta = (ReapplyCondition = "ReapplyLandscapeLayers"))
	TArray<FName> ExclusionLandscapeLayers;

	/** Specifies the minimum value above which the landscape exclusion layer weight value must be, in order for foliage instances to be excluded in a specific area */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Placement, meta = (UIMin=0, ClampMin = 0, UIMax = 1, ClampMax = 1, HideBehind = "ExclusionLandscapeLayers"))
	float MinimumExclusionLayerWeight;

	UPROPERTY()
	FName LandscapeLayer_DEPRECATED;
	
	/* If checked, an overlap test with existing world geometry is performed before each instance is placed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=Placement, meta=(ReapplyCondition="ReapplyCollisionWithWorld"))
	uint32 CollisionWithWorld:1;

	/* The foliage instance's collision bounding box will be scaled by the specified amount before performing the overlap check */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=Placement, meta=(HideBehind="CollisionWithWorld"))
	FVector CollisionScale;
		
	/** Line trace count to use around hit location when averaging normal */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Placement, meta=(EditCondition="AverageNormal"))
	int32 AverageNormalSampleCount;

	UPROPERTY()
	FBoxSphereBounds MeshBounds;

	// X, Y is origin position and Z is radius...
	UPROPERTY()
	FVector LowBoundOriginRadius;

public:
	// INSTANCE SETTINGS

	/** Mobility property to apply to foliage components */
	UPROPERTY(Category = InstanceSettings, EditAnywhere, BlueprintReadOnly)
	TEnumAsByte<EComponentMobility::Type> Mobility;

	/**
	 * The distance where instances will begin to fade out if using a PerInstanceFadeAmount material node. 0 disables.
	 * When the entire cluster is beyond this distance, the cluster is completely culled and not rendered at all.
	 */
	UPROPERTY(EditAnywhere, Category=InstanceSettings, meta=(UIMin=0))
	FInt32Interval CullDistance;

	/** Deprecated. Now use the Mobility setting to control static/dynamic lighting */
	UPROPERTY()
	uint32 bEnableStaticLighting_DEPRECATED : 1;

	/** Controls whether the foliage should cast a shadow or not. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=InstanceSettings)
	uint32 CastShadow:1;

	/** Controls whether the foliage should inject light into the Light Propagation Volume.  This flag is only used if CastShadow is true. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=InstanceSettings, meta=(EditCondition="CastShadow"))
	uint32 bAffectDynamicIndirectLighting:1;

	/** Controls whether the primitive should affect dynamic distance field lighting methods.  This flag is only used if CastShadow is true. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=InstanceSettings, meta=(EditCondition="CastShadow"))
	uint32 bAffectDistanceFieldLighting:1;

	/** Controls whether the foliage should cast shadows in the case of non precomputed shadowing.  This flag is only used if CastShadow is true. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=InstanceSettings, meta=(EditCondition="CastShadow"))
	uint32 bCastDynamicShadow:1;

	/** Whether the foliage should cast a static shadow from shadow casting lights.  This flag is only used if CastShadow is true. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=InstanceSettings, meta=(EditCondition="CastShadow"))
	uint32 bCastStaticShadow:1;

	/** Whether the object should cast contact shadows. This flag is only used if CastShadow is true. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=InstanceSettings, meta=(EditCondition = "CastShadow"))
	uint8 bCastContactShadow : 1;

	/** Whether this foliage should cast dynamic shadows as if it were a two sided material. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=InstanceSettings, meta=(EditCondition="bCastDynamicShadow"))
	uint32 bCastShadowAsTwoSided:1;

	/** Whether the foliage receives decals. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=InstanceSettings)
	uint32 bReceivesDecals : 1;

	/** Whether to override the lightmap resolution defined in the static mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=InstanceSettings, meta=(InlineEditConditionToggle))
	uint32 bOverrideLightMapRes:1;

	/** Control shadow invalidation behavior, in particular with respect to Virtual Shadow Maps and material effects like World Position Offset. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=InstanceSettings, meta=(EditCondition="CastShadow"))
	EShadowCacheInvalidationBehavior ShadowCacheInvalidationBehavior;

	/** Overrides the lightmap resolution defined in the static mesh */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=InstanceSettings, meta=(DisplayName="Light Map Resolution", EditCondition="bOverrideLightMapRes"))
	int32 OverriddenLightMapRes;

	/** Controls the type of lightmap used for this component. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=InstanceSettings)
	ELightmapType LightmapType;

	/**
	 * If enabled, foliage will render a pre-pass which allows it to occlude other primitives, and also allows 
	 * it to correctly receive DBuffer decals. Enabling this setting may have a negative performance impact.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = InstanceSettings)
	uint32 bUseAsOccluder : 1;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = InstanceSettings)
	uint8 bVisibleInRayTracing : 1;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = InstanceSettings)
	uint8 bVisibleInReflections : 1;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = InstanceSettings)
	uint8 bEvaluateWorldPositionOffset : 1;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = InstanceSettings, meta = (ClampMin=0))
	int32 WorldPositionOffsetDisableDistance;

	/** Used to forcefully disable pixel programmable rasterization of Nanite when the mesh is further than a given distance from the camera. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = InstanceSettings, meta = (ClampMin = 0))
	float NanitePixelProgrammableDistance = 0.0f;

	/** Custom collision for foliage */
	UPROPERTY(EditAnywhere, Category=InstanceSettings, meta=(HideObjectType=true))
	struct FBodyInstance BodyInstance;

	/** Force navmesh */
	UPROPERTY(EditAnywhere, Category=InstanceSettings, meta=(HideObjectType=true))
	TEnumAsByte<EHasCustomNavigableGeometry::Type> CustomNavigableGeometry;

	/**
	 * Lighting channels that placed foliage will be assigned. Lights with matching channels will affect the foliage.
	 * These channels only apply to opaque materials, direct lighting, and dynamic lighting and shadowing.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=InstanceSettings)
	FLightingChannels LightingChannels;

	/** If true, the foliage will be rendered in the CustomDepth pass (usually used for outlines) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=InstanceSettings, meta=(DisplayName = "Render CustomDepth Pass"))
	uint32 bRenderCustomDepth:1;

	/** Mask used for stencil buffer writes. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=InstanceSettings, meta = (editcondition = "bRenderCustomDepth"))
	ERendererStencilMask CustomDepthStencilWriteMask;

	/** Optionally write this 0-255 value to the stencil buffer in CustomDepth pass (Requires project setting or r.CustomDepth == 3) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=InstanceSettings,  meta=(UIMin = "0", UIMax = "255", editcondition = "bRenderCustomDepth", DisplayName = "CustomDepth Stencil Value"))
	int32 CustomDepthStencilValue;

	/**
	 * Translucent objects with a lower sort priority draw behind objects with a higher priority.
	 * Translucent objects with the same priority are rendered from back-to-front based on their bounds origin.
	 * This setting is also used to sort objects being drawn into a runtime virtual texture.
	 *
	 * Ignored if the object is not translucent.  The default priority is zero.
	 * Warning: This should never be set to a non-default value unless you know what you are doing, as it will prevent the renderer from sorting correctly.
	 * It is especially problematic on dynamic gameplay effects.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=InstanceSettings)
	int32 TranslucencySortPriority;

#if WITH_EDITORONLY_DATA
	/** Bitflag to represent in which editor views this foliage mesh is hidden. */
	UPROPERTY(transient)
	uint64 HiddenEditorViews;

	UPROPERTY(transient)
	uint32 IsSelected:1;
#endif
public:
	float GetRadius(bool bInSingleInstanceMode) const { return bInSingleInstanceMode && bSingleInstanceModeOverrideRadius ? SingleInstanceModeRadius : Radius; }

	// PROCEDURAL

	float GetSeedDensitySquared() const { return InitialSeedDensity * InitialSeedDensity; }
	FOLIAGE_API float GetMaxRadius() const;
	FOLIAGE_API float GetScaleForAge(const float Age) const;
	FOLIAGE_API float GetInitAge(FRandomStream& RandomStream) const;
	FOLIAGE_API float GetNextAge(const float CurrentAge, const int32 NumSteps) const;
	FOLIAGE_API bool GetSpawnsInShade() const;

	// COLLISION

	/** The CollisionRadius determines when two instances overlap. When two instances overlap a winner will be picked based on rules and priority. */
	UPROPERTY(Category=Procedural, EditAnywhere, meta=(Subcategory="Collision", ClampMin="0.0", UIMin="0.0"))
	float CollisionRadius;

	/** The ShadeRadius determines when two instances overlap. If an instance can grow in the shade this radius is ignored.*/
	UPROPERTY(Category=Procedural, EditAnywhere, meta=(Subcategory="Collision", ClampMin="0.0", UIMin="0.0"))
	float ShadeRadius;

	/** The number of times we age the species and spread its seeds. */
	UPROPERTY(Category=Procedural, EditAnywhere, meta=(Subcategory="Clustering", ClampMin="0", UIMin="0"))
	int32 NumSteps;

	// CLUSTERING

	/** Specifies the number of seeds to populate along 10 meters. The number is implicitly squared to cover a 10m x 10m area*/
	UPROPERTY(Category=Procedural, EditAnywhere, meta=(Subcategory="Clustering", ClampMin="0.0", UIMin="0.0"))
	float InitialSeedDensity;

	/** The average distance between the spreading instance and its seeds. For example, a tree with an AverageSpreadDistance 10 will ensure the average distance between the tree and its seeds is 10cm */
	UPROPERTY(Category=Procedural, EditAnywhere, meta=(Subcategory="Clustering", ClampMin="0.0", UIMin="0.0"))
	float AverageSpreadDistance;

	/** Specifies how much seed distance varies from the average. For example, a tree with an AverageSpreadDistance 10 and a SpreadVariance 1 will produce seeds with an average distance of 10cm plus or minus 1cm */
	UPROPERTY(Category=Procedural, EditAnywhere, meta=(Subcategory="Clustering", ClampMin="0.0", UIMin="0.0"))
	float SpreadVariance;

	/** The number of seeds an instance will spread in a single step of the simulation. */
	UPROPERTY(Category=Procedural, EditAnywhere, meta=(Subcategory="Clustering", ClampMin="0", UIMin="0"))
	int32 SeedsPerStep;

	/** The seed that determines placement of initial seeds. */
	UPROPERTY(Category=Procedural, EditAnywhere, meta=(Subcategory="Clustering"))
	int32 DistributionSeed;

	/** The max seed used to compute global offset. */
	UPROPERTY(Category=Procedural, EditAnywhere, meta=(Subcategory="Clustering"))
	float MaxInitialSeedOffset;

	// GROWTH

	/** If true, seeds of this type will ignore shade radius during overlap tests with other types. */
	UPROPERTY(Category=Procedural, EditAnywhere, meta=(Subcategory="Growth"))
	bool bCanGrowInShade;

	/** 
	 * Whether new seeds are spawned exclusively in shade. Occurs in a second pass after all types that do not spawn in shade have been simulated. 
	 * Only valid when CanGrowInShade is true. 
	 */
	UPROPERTY(Category=Procedural, EditAnywhere, meta=(Subcategory="Growth", EditCondition="bCanGrowInShade"))
	bool bSpawnsInShade;

	/** Allows a new seed to be older than 0 when created. New seeds will be randomly assigned an age in the range [0,MaxInitialAge] */
	UPROPERTY(Category=Procedural, EditAnywhere, meta=(Subcategory="Growth", ClampMin="0.0", UIMin="0.0"))
	float MaxInitialAge;

	/** Specifies the oldest a seed can be. After reaching this age the instance will still spread seeds, but will not get any older*/
	UPROPERTY(Category=Procedural, EditAnywhere, meta=(Subcategory="Growth", ClampMin="0.0", UIMin="0.0"))
	float MaxAge;

	/** 
	 * When two instances overlap we must determine which instance to remove. 
	 * The instance with a lower OverlapPriority will be removed. 
	 * In the case where OverlapPriority is the same regular simulation rules apply.
	 */
	UPROPERTY(Category=Procedural, EditAnywhere, meta=(Subcategory="Growth"))
	float OverlapPriority;

	/** The scale range of this type when being procedurally generated. Configured with the Scale Curve. */
	UPROPERTY(Category=Procedural, EditAnywhere, meta = (Subcategory = "Growth", ClampMin = "0.001", UIMin = "0.001"))
	FFloatInterval ProceduralScale;

	/** 
	 * Instance scale factor as a function of normalized age (i.e. Current Age / Max Age).
	 * X = 0 corresponds to Age = 0, X = 1 corresponds to Age = Max Age.
	 * Y = 0 corresponds to Min Scale, Y = 1 corresponds to Max Scale.
	 */
	UPROPERTY(Category = Procedural, EditAnywhere, meta = (Subcategory = "Growth", XAxisName = "Normalized Age", YAxisName = "Scale Factor"))
	FRuntimeFloatCurve ScaleCurve;

	UPROPERTY(Category = Procedural, EditAnywhere)
	FFoliageDensityFalloff DensityFalloff;

	UPROPERTY(Transient)
	int32 ChangeCount;

public:
	// REAPPLY EDIT CONDITIONS

	/** If checked, the density of foliage instances already placed will be adjusted by the density adjustment factor. */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyDensity:1;

	/** If checked, foliage instances not meeting the new Radius constraint will be removed */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyRadius:1;

	/** If checked, foliage instances will have their normal alignment adjusted by the Reapply tool */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyAlignToNormal:1;

	/** If checked, foliage instances will have their yaw adjusted by the Reapply tool */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyRandomYaw:1;

	/** If checked, foliage instances will have their scale adjusted to fit the specified scaling behavior by the Reapply tool */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyScaling:1;

	/** If checked, foliage instances will have their X scale adjusted by the Reapply tool */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyScaleX:1;

	/** If checked, foliage instances will have their Y scale adjusted by the Reapply tool */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyScaleY:1;

	/** If checked, foliage instances will have their Z scale adjusted by the Reapply tool */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyScaleZ:1;

	/** If checked, foliage instances will have their pitch adjusted by the Reapply tool */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyRandomPitchAngle:1;

	/** If checked, foliage instances not meeting the ground slope condition will be removed by the Reapply too */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyGroundSlope:1;

	/* If checked, foliage instances not meeting the valid Z height condition will be removed by the Reapply tool */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyHeight:1;

	/* If checked, foliage instances painted on areas that do not have the appropriate landscape layer painted will be removed by the Reapply tool */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyLandscapeLayers:1;

	/** If checked, foliage instances will have their Z offset adjusted by the Reapply tool */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyZOffset:1;

	/* If checked, foliage instances will have an overlap test with the world reapplied, and overlapping instances will be removed by the Reapply tool */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyCollisionWithWorld:1;

	/* If checked, foliage instances no longer matching the vertex color constraint will be removed by the Reapply too */
	UPROPERTY(EditDefaultsOnly, Category=Reapply)
	uint32 ReapplyVertexColorMask:1;

public:
	// SCALABILITY

	/**
	 * Whether this foliage type should be affected by the Engine Scalability system's Foliage scalability setting.
	 * Enable for detail meshes that don't really affect the game. Disable for anything important.
	 * Typically, this will be enabled for small meshes without collision (e.g. grass) and disabled for large meshes with collision (e.g. trees)
	 */
	UPROPERTY(EditAnywhere, Category=Scalability)
	uint32 bEnableDensityScaling:1;

	/**
	* Whether this foliage type should be discarded when CVarFoliageDiscardDataOnLoad is enabled.
	*/
	UPROPERTY(EditAnywhere, Category=Scalability)
	uint32 bEnableDiscardOnLoad:1;

	/*
	 * Whether this foliage type should be affected by the Engine's "foliage.CullDistanceScale" setting 
	 */
	UPROPERTY(EditAnywhere, Category = Scalability)
	uint32 bEnableCullDistanceScaling : 1;

public:
	// VIRTUAL TEXTURE

	/** 
	 * Array of runtime virtual textures into which we draw the instances. 
	 * The mesh material also needs to be set up to output to a virtual texture. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=VirtualTexture, meta = (DisplayName = "Draw in Virtual Textures"))
	TArray<TObjectPtr<URuntimeVirtualTexture>> RuntimeVirtualTextures;

	/**
	 * Number of lower mips in the runtime virtual texture to skip for rendering this primitive.
	 * Larger values reduce the effective draw distance in the runtime virtual texture.
	 * This culling method doesn't take into account primitive size or virtual texture size.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = VirtualTexture, meta = (DisplayName = "Virtual Texture Skip Mips", UIMin = "0", UIMax = "7"))
	int32 VirtualTextureCullMips = 0;

	/** Controls if this component draws in the main pass as well as in the virtual texture. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = VirtualTexture, meta = (DisplayName = "Draw in Main Pass"))
	ERuntimeVirtualTextureMainPassType VirtualTextureRenderPassType = ERuntimeVirtualTextureMainPassType::Exclusive;

public:
	// HLOD

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=HLOD)
	uint32 bIncludeInHLOD:1;
#endif // WITH_EDITORONLY_DATA

private:

#if WITH_EDITORONLY_DATA
	// Deprecated since FFoliageCustomVersion::FoliageTypeCustomization
	UPROPERTY()
	float ScaleMinX_DEPRECATED;

	UPROPERTY()
	float ScaleMinY_DEPRECATED;

	UPROPERTY()
	float ScaleMinZ_DEPRECATED;

	UPROPERTY()
	float ScaleMaxX_DEPRECATED;

	UPROPERTY()
	float ScaleMaxY_DEPRECATED;

	UPROPERTY()
	float ScaleMaxZ_DEPRECATED;

	UPROPERTY()
	float HeightMin_DEPRECATED;

	UPROPERTY()
	float HeightMax_DEPRECATED;

	UPROPERTY()
	float ZOffsetMin_DEPRECATED;

	UPROPERTY()
	float ZOffsetMax_DEPRECATED;

	UPROPERTY()
	int32 StartCullDistance_DEPRECATED;
	
	UPROPERTY()
	int32 EndCullDistance_DEPRECATED;

	UPROPERTY()
	uint32 UniformScale_DEPRECATED:1;

	UPROPERTY()
	uint32 LockScaleX_DEPRECATED:1;

	UPROPERTY()
	uint32 LockScaleY_DEPRECATED:1;

	UPROPERTY()
	uint32 LockScaleZ_DEPRECATED:1;

	UPROPERTY()
	float GroundSlope_DEPRECATED;
	
	UPROPERTY()
	float MinGroundSlope_DEPRECATED;

	UPROPERTY()
	float MinScale_DEPRECATED;

	UPROPERTY()
	float MaxScale_DEPRECATED;
#endif// WITH_EDITORONLY_DATA
};


