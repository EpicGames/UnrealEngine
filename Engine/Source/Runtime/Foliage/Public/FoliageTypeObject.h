// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "FoliageType_InstancedStaticMesh.h"
#include "FoliageTypeObject.generated.h"

/** A wrapper struct used to allow the use of either FoliageType assets or FoliageType blueprint classes */
USTRUCT()
struct FFoliageTypeObject
{
	GENERATED_USTRUCT_BODY()

	FFoliageTypeObject()
		: FoliageTypeObject(nullptr), TypeInstance(nullptr), bIsAsset(false) {}

	/** Refreshes the type instance based on the assigned object. Intended to be called after some change is made. */
	FOLIAGE_API void RefreshInstance();

	/** Gets the instance of this foliage type. Creates the instance if needed. */
	FOLIAGE_API const UFoliageType* GetInstance();

	/** Gets the instance of this foliage type. */
	FOLIAGE_API const UFoliageType* GetInstance() const;

	/** @return Whether this would return a valid instance */
	FOLIAGE_API bool ContainsValidInstance() const;

	/** @return Whether any foliage type is assigned at all */
	FOLIAGE_API bool HasFoliageType() const;

	FOLIAGE_API bool IsDirty() const;
	FOLIAGE_API void SetClean();

#if WITH_EDITORONLY_DATA
	FOLIAGE_API void PostSerialize(const FArchive& Ar);
#endif

private:
	/** The foliage type that will be spawned by the procedural foliage simulation */
	UPROPERTY(Category = ProceduralFoliageSimulation, EditAnywhere, meta=(AllowedClasses="/Script/Foliage.FoliageType_InstancedStaticMesh,/Script/Foliage.FoliageType_Actor,/Script/Engine.Blueprint", DisplayThumbnail="true", ThumbnailSize="X=40 Y=40"))
	TObjectPtr<UObject> FoliageTypeObject;

	/** The actual instance of the foliage type that is used for spawning */
	UPROPERTY(transient)
	TObjectPtr<UFoliageType> TypeInstance;

	/** Whether this contains an asset object (as opposed to a BP class) */
	UPROPERTY()
	bool bIsAsset;

	UPROPERTY()
	TSubclassOf<UFoliageType_InstancedStaticMesh> Type_DEPRECATED;
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FFoliageTypeObject> : public TStructOpsTypeTraitsBase2<FFoliageTypeObject>
{
	enum
	{
		WithPostSerialize = true,
	};
};
#endif
