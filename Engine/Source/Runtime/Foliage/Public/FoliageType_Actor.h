// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SubclassOf.h"
#include "FoliageType.h"
#include "FoliageType_Actor.generated.h"

class UFoliageInstancedStaticMeshComponent;

UCLASS(hidecategories = Object, editinlinenew, MinimalAPI)
class UFoliageType_Actor : public UFoliageType
{
	GENERATED_UCLASS_BODY()
	
	UPROPERTY(EditAnywhere, Category = Actor)
	TSubclassOf<AActor> ActorClass;
			
	UPROPERTY(EditAnywhere, Category = Actor)
	bool bShouldAttachToBaseComponent;

	UPROPERTY(EditAnywhere, Category = Mesh, meta = (ToolTip = "If enabled, will place an instanced static mesh representation of this actor without placing an actual actor"))
	bool bStaticMeshOnly;
		
	UPROPERTY(EditAnywhere, Category = Mesh, meta = (EditCondition = "bStaticMeshOnly"))
	TSubclassOf<UFoliageInstancedStaticMeshComponent> StaticMeshOnlyComponentClass;

	virtual UObject* GetSource() const override { return ActorClass; }

#if WITH_EDITOR
	virtual FString GetDefaultNewAssetName() const override
	{
		return TEXT("NewActorFoliage");
	}

	virtual void UpdateBounds();
	virtual bool IsSourcePropertyChange(const FProperty* Property) const override
	{
		return Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFoliageType_Actor, ActorClass);
	}
	virtual void SetSource(UObject* InSource) override
	{
		ActorClass = Cast<UClass>(InSource);
		UpdateBounds();
	}
#endif
};