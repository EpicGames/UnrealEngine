// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "FoliageType.h"
#include "FoliageInstancedStaticMeshComponent.h"
#include "FoliageType_InstancedStaticMesh.generated.h"

UCLASS(hidecategories=Object, editinlinenew, MinimalAPI)
class UFoliageType_InstancedStaticMesh : public UFoliageType
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Mesh, meta=(DisplayThumbnail="true"))
	TObjectPtr<UStaticMesh> Mesh;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Mesh, Meta = (ToolTip = "Material overrides for foliage instances."))
	TArray<TObjectPtr<class UMaterialInterface>> OverrideMaterials;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Mesh, Meta = (ToolTip = "Nanite material overrides for foliage instances."))
	TArray<TObjectPtr<class UMaterialInterface>> NaniteOverrideMaterials;
		
	/** The component class to use for foliage instances. 
	  * You can make a Blueprint subclass of FoliageInstancedStaticMeshComponent to implement custom behavior and assign that class here. */
	UPROPERTY(EditAnywhere, Category = Mesh)
	TSubclassOf<UFoliageInstancedStaticMeshComponent> ComponentClass;
		
	UStaticMesh* GetStaticMesh() const
	{
		return Mesh;
	}

	UClass* GetComponentClass() const
	{
		return *ComponentClass;
	}

	virtual UObject* GetSource() const override;

#if WITH_EDITOR
	virtual FString GetDefaultNewAssetName() const override
	{
		return TEXT("NewInstancedStaticMeshFoliage");
	}

	virtual void UpdateBounds() override;
	virtual bool IsSourcePropertyChange(const FProperty* Property) const override
	{
		return Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFoliageType_InstancedStaticMesh, Mesh);
	}
	virtual void SetSource(UObject* InSource) override;

	void SetStaticMesh(UStaticMesh* InStaticMesh)
	{
		Mesh = InStaticMesh;
		UpdateBounds();
	}
#endif
};
