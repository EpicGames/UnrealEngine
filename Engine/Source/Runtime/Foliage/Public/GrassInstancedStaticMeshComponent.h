// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "GrassInstancedStaticMeshComponent.generated.h"

UCLASS(ClassGroup = Foliage, MinimalAPI, Hidden)
class UGrassInstancedStaticMeshComponent : public UHierarchicalInstancedStaticMeshComponent
{
	GENERATED_UCLASS_BODY()

public:
	static FOLIAGE_API void BuildTreeAnyThread(TArray<FMatrix>& InstanceTransforms, TArray<float>& InstanceCustomDataFloats, int32 NumCustomDataFloats, const FBox& MeshBox, TArray<FClusterNode>& OutClusterTree, TArray<int32>& OutSortedInstances, TArray<int32>& OutInstanceReorderTable, int32& OutOcclusionLayerNum, int32 MaxInstancesPerLeaf, bool InGenerateInstanceScalingRange);
	
	FOLIAGE_API void AcceptPrebuiltTree(TArray<FClusterNode>& InClusterTree, int32 InOcclusionLayerNumNodes, int32 InNumBuiltRenderInstances, FStaticMeshInstanceData* InSharedInstanceBufferData);

	FOLIAGE_API bool SupportsWorldPositionOffsetVelocity() const override;

	int32 GetNumRenderInstances() const override { return NumBuiltRenderInstances; }

private:
	virtual void BuildTree() override final;
	virtual void BuildTreeAsync() override final;
	UE_DEPRECATED(5.7, "Please use BuildComponentInstanceData with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	virtual void BuildComponentInstanceData(ERHIFeatureLevel::Type FeatureLevel, FInstanceUpdateComponentDesc& OutData) override final;
	virtual void BuildComponentInstanceData(EShaderPlatform InShaderPlatform, FInstanceUpdateComponentDesc& OutData) override final;
};

