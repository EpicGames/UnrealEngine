// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
InstancedFoliage.h: Instanced foliage type definitions.
=============================================================================*/
#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Containers/ArrayView.h"
#include "FoliageInstanceBase.h"
#include "Instances/InstancedPlacementHash.h"

class AInstancedFoliageActor;
class UActorComponent;
class UFoliageType;
class UInstancedStaticMeshComponent;
class UHierarchicalInstancedStaticMeshComponent;
class UFoliageType_InstancedStaticMesh;
class UPrimitiveComponent;
class UStaticMesh;
struct FSMInstanceId;

#if WITH_EDITORONLY_DATA
using FFoliageInstanceHash = FInstancedPlacementHash;
#endif

DECLARE_LOG_CATEGORY_EXTERN(LogInstancedFoliage, Log, All);

namespace FoliageElementUtil
{
	FOLIAGE_API bool FoliageInstanceElementsEnabled();
}

/**
* Flags stored with each instance
*/
enum EFoliageInstanceFlags
{
	FOLIAGE_AlignToNormal = 0x00000001,
	FOLIAGE_NoRandomYaw = 0x00000002,
	FOLIAGE_Readjusted = 0x00000004,
	FOLIAGE_InstanceDeleted = 0x00000008,	// Used only for migration from pre-HierarchicalISM foliage.
};

/**
*	FFoliageInstancePlacementInfo - placement info an individual instance
*/
struct FFoliageInstancePlacementInfo
{
	FVector Location;
	FRotator Rotation;
	FRotator PreAlignRotation;
	FVector3f DrawScale3D;
	float ZOffset;
	uint32 Flags;

	FFoliageInstancePlacementInfo()
		: Location(0.f, 0.f, 0.f)
		, Rotation(0, 0, 0)
		, PreAlignRotation(0, 0, 0)
		, DrawScale3D(1.f, 1.f, 1.f)
		, ZOffset(0.f)
		, Flags(0)
	{}
};

/**
*	Legacy instance
*/
struct FFoliageInstance_Deprecated : public FFoliageInstancePlacementInfo
{
	UActorComponent* Base;
	FGuid ProceduralGuid;
	friend FArchive& operator<<(FArchive& Ar, FFoliageInstance_Deprecated& Instance);
};

/**
*	FFoliageInstance - editor info an individual instance
*/
struct FFoliageInstance : public FFoliageInstancePlacementInfo
{
	// ID of base this instance was painted on
	FFoliageInstanceBaseId BaseId;

	FGuid ProceduralGuid;

	UActorComponent* BaseComponent;

	FFoliageInstance()
		: BaseId(0)
		, BaseComponent(nullptr)
	{}

	friend FArchive& operator<<(FArchive& Ar, FFoliageInstance& Instance);

	FTransform GetInstanceWorldTransform() const
	{
		return FTransform(Rotation, Location, FVector(DrawScale3D));
	}

	void SetInstanceWorldTransform(const FTransform& Transform)
	{
		Location = Transform.GetTranslation();
		Rotation = Transform.Rotator();
		DrawScale3D = FVector3f(Transform.GetScale3D());
	}

	void AlignToNormal(const FVector& InNormal, float AlignMaxAngle = 0.f)
	{
		Flags |= FOLIAGE_AlignToNormal;

		FRotator AlignRotation = InNormal.Rotation();
		// Static meshes are authored along the vertical axis rather than the X axis, so we add 90 degrees to the static mesh's Pitch.
		AlignRotation.Pitch -= 90.f;
		// Clamp its value inside +/- one rotation
		AlignRotation.Pitch = FRotator::NormalizeAxis(AlignRotation.Pitch);

		// limit the maximum pitch angle if it's > 0.
		if (AlignMaxAngle > 0.f)
		{
			int32 MaxPitch = static_cast<int32>(AlignMaxAngle);
			if (AlignRotation.Pitch > MaxPitch)
			{
				AlignRotation.Pitch = MaxPitch;
			}
			else if (AlignRotation.Pitch < -MaxPitch)
			{
				AlignRotation.Pitch = -MaxPitch;
			}
		}

		PreAlignRotation = Rotation;
		Rotation = FRotator(FQuat(AlignRotation) * FQuat(Rotation));
	}
};

struct FFoliageMeshInfo_Deprecated
{
	UHierarchicalInstancedStaticMeshComponent* Component;

#if WITH_EDITORONLY_DATA
	// Allows us to detect if FoliageType was updated while this level wasn't loaded
	FGuid FoliageTypeUpdateGuid;

	// Editor-only placed instances
	TArray<FFoliageInstance_Deprecated> Instances;
#endif

	FFoliageMeshInfo_Deprecated()
		: Component(nullptr)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FFoliageMeshInfo_Deprecated& MeshInfo);
};

/**
*	FFoliageInfo - editor info for all matching foliage meshes
*/
struct FFoliageMeshInfo_Deprecated2
{
	UHierarchicalInstancedStaticMeshComponent* Component;

#if WITH_EDITORONLY_DATA
	// Allows us to detect if FoliageType was updated while this level wasn't loaded
	FGuid FoliageTypeUpdateGuid;

	// Editor-only placed instances
	TArray<FFoliageInstance> Instances;
#endif


	FFoliageMeshInfo_Deprecated2();
	
	friend FArchive& operator<<(FArchive& Ar, FFoliageMeshInfo_Deprecated2& MeshInfo);
};

///
/// EFoliageImplType
///
enum class EFoliageImplType : uint8
{
	Unknown = 0,
	StaticMesh = 1,
	Actor = 2,
	ISMActor = 3
};

struct FFoliageInfo;

///
/// FFoliageInfoImpl
///
struct FFoliageImpl
{
	FFoliageImpl(FFoliageInfo* InInfo) 
#if WITH_EDITORONLY_DATA
		: Info(InInfo)
#endif
	{}
	virtual ~FFoliageImpl() = default;

	virtual void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector) {}
	virtual void Serialize(FArchive& Ar) = 0;
	virtual void PostSerialize(FArchive& Ar) {}
	virtual void PostLoad() {}

#if WITH_EDITORONLY_DATA
	// Not serialized but FFoliageInfo will make sure it stays valid (mostly Undo/Redo)
	FFoliageInfo* Info;
#endif

#if WITH_EDITOR
	virtual bool IsInitialized() const = 0;
	virtual void Initialize(const UFoliageType* FoliageType) = 0;
	virtual void Uninitialize() = 0;
	virtual void Reapply(const UFoliageType* FoliageType) = 0;
	virtual int32 GetInstanceCount() const = 0;
	virtual void PreAddInstances(const UFoliageType* FoliageType, int32 Count) = 0;
	virtual void AddInstance(const FFoliageInstance& NewInstance) = 0;
	virtual void RemoveInstance(int32 InstanceIndex) = 0;
	virtual void MoveInstance(int32 InstanceIndex, UObject*& OutInstanceImplementation) { RemoveInstance(InstanceIndex); }
	virtual void AddExistingInstance(const FFoliageInstance& ExistingInstance, UObject* InstanceImplementation) { AddInstance(ExistingInstance); }
	virtual void SetInstanceWorldTransform(int32 InstanceIndex, const FTransform& Transform, bool bTeleport) = 0;
	virtual FTransform GetInstanceWorldTransform(int32 InstanceIndex) const = 0;
	virtual void PostUpdateInstances() {}
	virtual void PreMoveInstances(TArrayView<const int32> InInstancesMoved) {}
	virtual void PostMoveInstances(TArrayView<const int32> InInstancesMoved, bool bFinished) {}
	virtual bool IsOwnedComponent(const UPrimitiveComponent* PrimitiveComponent) const = 0;
	
	virtual void SelectAllInstances(bool bSelect) = 0;
	virtual void SelectInstance(bool bSelect, int32 Index) = 0;
	virtual void SelectInstances(bool bSelect, const TSet<int32>& SelectedIndices) = 0;
	virtual int32 GetInstanceIndexFrom(const UPrimitiveComponent* PrimitiveComponent, int32 ComponentIndex) const = 0;
	virtual FBox GetSelectionBoundingBox(const TSet<int32>& SelectedIndices) const = 0;
	virtual void ApplySelection(bool bApply, const TSet<int32>& SelectedIndices) = 0;
	virtual void ClearSelection(const TSet<int32>& SelectedIndices) = 0;

	virtual void ForEachSMInstance(TFunctionRef<bool(FSMInstanceId)> Callback) const {}
	virtual void ForEachSMInstance(int32 InstanceIndex, TFunctionRef<bool(FSMInstanceId)> Callback) const {}

	virtual void BeginUpdate() {}
	virtual void EndUpdate() {}
	virtual void Refresh(bool Async, bool Force) {}
	virtual void OnHiddenEditorViewMaskChanged(uint64 InHiddenEditorViews) = 0;
	virtual void PreEditUndo(UFoliageType* FoliageType) {}
	virtual void PostEditUndo(FFoliageInfo* InInfo, UFoliageType* FoliageType) { Info = InInfo; }
	virtual void NotifyFoliageTypeWillChange(UFoliageType* FoliageType) {}

	// Return true if implementation needs to change 
	virtual bool NotifyFoliageTypeChanged(UFoliageType* FoliageType, bool bSourceChanged) = 0;
	virtual void EnterEditMode() {}
	virtual void ExitEditMode() {}
	virtual bool ShouldAttachToBaseComponent() const { return true; }

	AInstancedFoliageActor* GetIFA() const;
	FFoliageInfo* GetInfo() const { check(Info); return Info; }
#endif

	virtual int32 GetOverlappingSphereCount(const FSphere& Sphere) const { return 0; }
	virtual int32 GetOverlappingBoxCount(const FBox& Box) const { return 0; }
	virtual void GetOverlappingBoxTransforms(const FBox& Box, TArray<FTransform>& OutTransforms) const { }
	virtual void GetOverlappingMeshCount(const FSphere& Sphere, TMap<UStaticMesh*, int32>& OutCounts) const { }
};

///
/// FFoliageInfo
///
struct FFoliageInfo
{
	EFoliageImplType Type;
	TUniquePtr<FFoliageImpl> Implementation;

#if WITH_EDITORONLY_DATA
	// Owning IFA
	AInstancedFoliageActor* IFA;

	// Allows us to detect if FoliageType was updated while this level wasn't loaded
	FGuid FoliageTypeUpdateGuid;

	// Editor-only placed instances
	TArray<FFoliageInstance> Instances;

	// Transient, editor-only locality hash of instances
	TUniquePtr<FFoliageInstanceHash> InstanceHash;

	// Transient, editor-only set of instances per component
	TMap<FFoliageInstanceBaseId, TSet<int32>> ComponentHash;

	// Transient, editor-only list of selected instances.
	TSet<int32> SelectedIndices;

	// Moving instances
	TSet<int32> MovingInstances;
#endif

	FOLIAGE_API FFoliageInfo();
	FOLIAGE_API ~FFoliageInfo();

	FFoliageInfo(FFoliageInfo&& Other) = default;
	FFoliageInfo& operator=(FFoliageInfo&& Other) = default;

	// Will only return a valid component in the case of non-actor foliage
	FOLIAGE_API UHierarchicalInstancedStaticMeshComponent* GetComponent() const;

	void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	void PostSerialize(FArchive& Ar);
	void PostLoad();

	FOLIAGE_API void CreateImplementation(EFoliageImplType InType);
	FOLIAGE_API void Initialize(const UFoliageType* FoliageType);
	FOLIAGE_API void Uninitialize();
	FOLIAGE_API bool IsInitialized() const;
		
	int32 GetOverlappingSphereCount(const FSphere& Sphere) const;
	int32 GetOverlappingBoxCount(const FBox& Box) const;
	void GetOverlappingBoxTransforms(const FBox& Box, TArray<FTransform>& OutTransforms) const;
	void GetOverlappingMeshCount(const FSphere& Sphere, TMap<UStaticMesh*, int32>& OutCounts) const;
	
#if WITH_EDITOR
	
	FOLIAGE_API EFoliageImplType GetImplementationType(const UFoliageType* FoliageType) const;
	FOLIAGE_API void CreateImplementation(const UFoliageType* FoliageType);
	FOLIAGE_API void NotifyFoliageTypeWillChange(UFoliageType* FoliageType);
	FOLIAGE_API void NotifyFoliageTypeChanged(UFoliageType* FoliageType, bool bSourceChanged);
	
	FOLIAGE_API void ClearSelection();

	FOLIAGE_API void SetInstanceWorldTransform(int32 InstanceIndex, const FTransform& Transform, bool bTeleport);

	FOLIAGE_API void SetRandomSeed(int32 seed);
	FOLIAGE_API void AddInstance(const UFoliageType* InSettings, const FFoliageInstance& InNewInstance);
	FOLIAGE_API void AddInstance(const UFoliageType* InSettings, const FFoliageInstance& InNewInstance, UActorComponent* InBaseComponent);
	FOLIAGE_API void AddInstances(const UFoliageType* InSettings, const TArray<const FFoliageInstance*>& InNewInstances);
	FOLIAGE_API void ReserveAdditionalInstances(const UFoliageType* InSettings, uint32 ReserveNum);

	FOLIAGE_API void RemoveInstances(TArrayView<const int32> InInstancesToRemove, bool RebuildFoliageTree);

	FOLIAGE_API void MoveInstances(AInstancedFoliageActor* InToIFA, const TSet<int32>& InInstancesToMove, bool bKeepSelection);

	// Apply changes in the FoliageType to the component
	FOLIAGE_API void PreMoveInstances(TArrayView<const int32> InInstancesMoved);
	FOLIAGE_API void PostMoveInstances(TArrayView<const int32> InInstancesMoved, bool bFinished = false);
	FOLIAGE_API void PostUpdateInstances(TArrayView<const int32>, bool bReAddToHash = false, bool InUpdateSelection = false);
	FOLIAGE_API void DuplicateInstances(UFoliageType* InSettings, TArrayView<const int32> InInstancesToDuplicate);
	FOLIAGE_API void GetInstancesInsideBounds(const FBox& Box, TArray<int32>& OutInstances) const;
	FOLIAGE_API void GetInstancesInsideSphere(const FSphere& Sphere, TArray<int32>& OutInstances) const;
	FOLIAGE_API void GetInstanceAtLocation(const FVector& Location, int32& OutInstance, bool& bOutSucess) const;
	FOLIAGE_API bool CheckForOverlappingSphere(const FSphere& Sphere) const;
	FOLIAGE_API bool CheckForOverlappingInstanceExcluding(int32 TestInstanceIdx, float Radius, TSet<int32>& ExcludeInstances) const;
	FOLIAGE_API TArray<int32> GetInstancesOverlappingBox(const FBox& Box) const;

	// Destroy existing clusters and reassign all instances to new clusters
	FOLIAGE_API void ReallocateClusters(UFoliageType* InSettings);

	FOLIAGE_API void SelectInstances(bool bSelect, TArrayView<const int32> Instances);

	FOLIAGE_API void SelectInstances(bool bSelect);

	FOLIAGE_API FBox GetSelectionBoundingBox() const;

	// Get the number of placed instances
	FOLIAGE_API int32 GetPlacedInstanceCount() const;

	FOLIAGE_API void AddToBaseHash(int32 InstanceIdx);
	FOLIAGE_API void RemoveFromBaseHash(int32 InstanceIdx);
	FOLIAGE_API void RecomputeHash();
	bool ShouldAttachToBaseComponent() const { return Implementation->ShouldAttachToBaseComponent(); }

	// For debugging. Validate state after editing.
	void CheckValid();
	
	void ForEachSMInstance(TFunctionRef<bool(FSMInstanceId)> Callback) const;
	void ForEachSMInstance(int32 InstanceIndex, TFunctionRef<bool(FSMInstanceId)> Callback) const;

	FOLIAGE_API void Refresh(bool Async, bool Force);
	FOLIAGE_API void OnHiddenEditorViewMaskChanged(uint64 InHiddenEditorViews);
	FOLIAGE_API void PostEditUndo(UFoliageType* FoliageType);
	FOLIAGE_API void PreEditUndo(UFoliageType* FoliageType);
	FOLIAGE_API void EnterEditMode();
	FOLIAGE_API void ExitEditMode();

	FOLIAGE_API void RemoveBaseComponentOnInstances();
	FOLIAGE_API void IncludeActor(const UFoliageType* FoliageType, AActor* InActor);
	FOLIAGE_API void ExcludeActors();

	FOLIAGE_API FBox GetApproximatedInstanceBounds() const;
#endif

	FOLIAGE_API friend FArchive& operator<<(FArchive& Ar, FFoliageInfo& MeshInfo);

	// Non-copyable
	FFoliageInfo(const FFoliageInfo&) = delete;
	FFoliageInfo& operator=(const FFoliageInfo&) = delete;

private:
	using FAddImplementationFunc = TFunctionRef<void(FFoliageImpl*, AInstancedFoliageActor*, const FFoliageInstance&)>;
	void AddInstancesImpl(const UFoliageType* InSettings, const TArray<const FFoliageInstance*>& InNewInstances, FFoliageInfo::FAddImplementationFunc ImplementationFunc);
	void AddInstanceImpl(const FFoliageInstance& InNewInstance, FAddImplementationFunc ImplementationFunc);

	using FRemoveImplementationFunc = TFunctionRef<void(FFoliageImpl*, int32)>;
	void RemoveInstancesImpl(TArrayView<const int32> InInstancesToRemove, bool RebuildFoliageTree, FRemoveImplementationFunc ImplementationFunc);
};

struct FFoliageInstanceId
{
	explicit operator bool() const
	{
		return Info != nullptr
			&& Index != INDEX_NONE;
	}

	bool operator==(const FFoliageInstanceId& InRHS) const
	{
		return Info == InRHS.Info
			&& Index == InRHS.Index;
	}

	bool operator!=(const FFoliageInstanceId& InRHS) const
	{
		return !(*this == InRHS);
	}

#if WITH_EDITOR
	FFoliageInstance* GetInstance() const
	{
		return (Info && Info->Instances.IsValidIndex(Index))
			? &Info->Instances[Index]
			: nullptr;
	}

	FFoliageInstance& GetInstanceChecked() const
	{
		FFoliageInstance* Instance = GetInstance();
		check(Instance);
		return *Instance;
	}
#endif

	friend inline uint32 GetTypeHash(const FFoliageInstanceId& InId)
	{
		return HashCombine(GetTypeHash(InId.Index), GetTypeHash(InId.Info));
	}

	FFoliageInfo* Info = nullptr;
	int32 Index = INDEX_NONE;
};

/** This is kind of a hack, but is needed right now for backwards compat of code. We use it to describe the placement mode (procedural vs manual)*/
namespace EFoliagePlacementMode
{
	enum Type
	{
		Manual = 0,
		Procedural = 1,
	};

}

/** Used to define a vector along which we'd like to spawn an instance. */
struct FDesiredFoliageInstance
{
	FDesiredFoliageInstance()
		: FoliageType(nullptr)
		, StartTrace(ForceInit)
		, EndTrace(ForceInit)
		, Rotation(ForceInit)
		, TraceRadius(0.f)
		, Age(0.f)
		, PlacementMode(EFoliagePlacementMode::Manual)
	{

	}

	FDesiredFoliageInstance(const FVector& InStartTrace, const FVector& InEndTrace, const UFoliageType* InFoliageType, const float InTraceRadius = 0.f)
		: FoliageType(InFoliageType)
		, StartTrace(InStartTrace)
		, EndTrace(InEndTrace)
		, Rotation(ForceInit)
		, TraceRadius(InTraceRadius)
		, Age(0.f)
		, PlacementMode(EFoliagePlacementMode::Manual)
	{
	}

	const UFoliageType* FoliageType;
	FGuid ProceduralGuid;
	FVector StartTrace;
	FVector EndTrace;
	FQuat Rotation;
	float TraceRadius;
	float Age;
	const struct FBodyInstance* ProceduralVolumeBodyInstance;
	EFoliagePlacementMode::Type PlacementMode;
};

#if WITH_EDITOR
// Struct to hold potential instances we've sampled
struct FPotentialInstance
{
	FVector HitLocation;
	FVector HitNormal;
	UPrimitiveComponent* HitComponent;
	float HitWeight;
	FDesiredFoliageInstance DesiredInstance;

	FOLIAGE_API FPotentialInstance(FVector InHitLocation, FVector InHitNormal, UPrimitiveComponent* InHitComponent, float InHitWeight, const FDesiredFoliageInstance& InDesiredInstance = FDesiredFoliageInstance());
	FOLIAGE_API bool PlaceInstance(const UWorld* InWorld, const UFoliageType* Settings, FFoliageInstance& Inst, bool bSkipCollision = false);
};
#endif
