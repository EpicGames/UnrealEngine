// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "FoliageType_InstancedStaticMesh.h"
#include "FoliageInstanceBase.h"
#include "InstancedFoliage.h"
#include "InstancedFoliageCustomVersion.h"
#include "ISMPartition/ISMPartitionActor.h"

#include "InstancedFoliageActor.generated.h"

#if WITH_EDITOR
class UProceduralFoliageComponent;
struct FActorPartitionIdentifier;
#endif

// Function for filtering out hit components during FoliageTrace
typedef TFunction<bool(const UPrimitiveComponent*)> FFoliageTraceFilterFunc;

UCLASS(notplaceable, hidecategories = (Object, Rendering, Mobility), MinimalAPI, NotBlueprintable)
class AInstancedFoliageActor : public AISMPartitionActor
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITORONLY_DATA
	// Cross level references cache for instances base
	FFoliageInstanceBaseCache InstanceBaseCache;

	UActorComponent* GetBaseComponentFromBaseId(const FFoliageInstanceBaseId& BaseId) const;
#endif// WITH_EDITORONLY_DATA

private:
	friend struct FFoliageInstanceBaseCache;
	
	TMap<TObjectPtr<UFoliageType>, TUniqueObj<FFoliageInfo>> FoliageInfos;

public:
	FOLIAGE_API bool ForEachFoliageInfo(TFunctionRef<bool(UFoliageType* FoliageType, FFoliageInfo& FoliageInfo)> InOperation);
	const TMap<UFoliageType*, TUniqueObj<FFoliageInfo>>& GetFoliageInfos() const { return ObjectPtrDecay(FoliageInfos); }
	FOLIAGE_API TUniqueObj<FFoliageInfo>& AddFoliageInfo(UFoliageType* FoliageType);
	FOLIAGE_API TUniqueObj<FFoliageInfo>& AddFoliageInfo(UFoliageType* FoliageType, TUniqueObj<FFoliageInfo>&& FoliageInfo);
	FOLIAGE_API bool RemoveFoliageInfoAndCopyValue(UFoliageType* FoliageType, TUniqueObj<FFoliageInfo>& OutFoliageInfo);

	//~ Begin UObject Interface.
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	FOLIAGE_API static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface. 

	//~ Begin AActor Interface.
	// we don't want to have our components automatically destroyed by the Blueprint code
#if WITH_EDITOR
	virtual void RerunConstructionScripts() override {}
#endif
	virtual bool IsLevelBoundsRelevant() const override { return false; }

public:
#if WITH_EDITOR
	bool CanDeleteSelectedActor(FText& OutReason) const override { return true; }
	bool CanEditFoliageInstance(const FFoliageInstanceId& InstanceId) const;
	bool CanMoveFoliageInstance(const FFoliageInstanceId& InstanceId, const ETypedElementWorldType WorldType) const;
	bool GetFoliageInstanceTransform(const FFoliageInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace) const;
	bool SetFoliageInstanceTransform(const FFoliageInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace, bool bTeleport);
	void NotifyFoliageInstanceMovementStarted(const FFoliageInstanceId& InstanceId);
	void NotifyFoliageInstanceMovementOngoing(const FFoliageInstanceId& InstanceId);
	void NotifyFoliageInstanceMovementEnded(const FFoliageInstanceId& InstanceId);
	void NotifyFoliageInstanceSelectionChanged(const FFoliageInstanceId& InstanceId, const bool bIsSelected);
	bool DeleteFoliageInstances(TArrayView<const FFoliageInstanceId> InstanceIds);
	bool DuplicateFoliageInstances(TArrayView<const FFoliageInstanceId> InstanceIds, TArray<FFoliageInstanceId>& OutNewInstanceIds);

	UFoliageType* GetFoliageTypeForInfo(const FFoliageInfo* FoliageInfo) const;
#endif

protected:
#if WITH_EDITOR
	void HandleFoliageInstancePreMove(const FFoliageInstanceId& InstanceId);
	void HandleFoliageInstancePostMove(const FFoliageInstanceId& InstanceId);
#endif

	//~ ISMInstanceManagerProvider interface
	virtual ISMInstanceManager* GetSMInstanceManager(const FSMInstanceId& InstanceId) override;

	// Default InternalTakeRadialDamage behavior finds and scales damage for the closest component which isn't appropriate for foliage.
	virtual float InternalTakeRadialDamage(float Damage, struct FRadialDamageEvent const& RadialDamageEvent, class AController* EventInstigator, AActor* DamageCauser) override;

public:
#if WITH_EDITOR
	FOLIAGE_API void EnterEditMode();
	FOLIAGE_API void ExitEditMode();

	virtual void PostInitProperties() override;
	virtual void BeginDestroy() override;
	virtual void Destroyed() override;
	virtual bool IsListedInSceneOutliner() const override;
	
	FOLIAGE_API void CleanupDeletedFoliageType();
	FOLIAGE_API void DetectFoliageTypeChangeAndUpdate();

	virtual uint32 GetDefaultGridSize(UWorld* InWorld) const override;
	virtual bool ShouldIncludeGridSizeInName(UWorld* InWorld, const FActorPartitionIdentifier& InIdentifier) const override;

	// Delegate type for selection change events
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSelectionChanged, bool, const TArray<AActor*>&);
	FOLIAGE_API static FOnSelectionChanged SelectionChanged;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnInstanceCoundChanged, const UFoliageType*);
	FOLIAGE_API static FOnInstanceCoundChanged InstanceCountChanged;
#endif
	//~ End AActor Interface.


	// Performs a reverse lookup from a source object to a local foliage type (i.e. the foliage type owned exclusively by this IFA)
	FOLIAGE_API UFoliageType* GetLocalFoliageTypeForSource(const UObject* InSource, FFoliageInfo** OutMeshInfo = nullptr);
	
	// Performs a reverse lookup from a source object to all the foliage types that are currently using that object (includes assets and blueprint classes)
	FOLIAGE_API void GetAllFoliageTypesForSource(const UObject* InSource, TArray<const UFoliageType*>& OutFoliageTypes);
	
	FOLIAGE_API FFoliageInfo* FindFoliageTypeOfClass(TSubclassOf<UFoliageType_InstancedStaticMesh> Class);

	// Finds the number of instances overlapping with the sphere. 
	FOLIAGE_API int32 GetOverlappingSphereCount(const UFoliageType* FoliageType, const FSphere& Sphere) const;
	// Finds the number of instances overlapping with the box. 
	FOLIAGE_API int32 GetOverlappingBoxCount(const UFoliageType* FoliageType, const FBox& Box) const;
	// Finds all instances in the provided box and get their transforms
	FOLIAGE_API void GetOverlappingBoxTransforms(const UFoliageType* FoliageType, const FBox& Box, TArray<FTransform>& OutTransforms) const;
	// Perf Warnin: potentially slow! Dev-only use recommended.
	// Returns list of meshes and counts for all nearby instances. OutCounts accumulates between runs.
	FOLIAGE_API void GetOverlappingMeshCounts(const FSphere& Sphere, TMap<UStaticMesh*, int32>& OutCounts) const;

	// Finds a mesh entry
	FOLIAGE_API FFoliageInfo* FindInfo(const UFoliageType* InType);

	// Finds a mesh entry
	FOLIAGE_API const FFoliageInfo* FindInfo(const UFoliageType* InType) const;

	/**
	* Get the instanced foliage actor for the current streaming level.
	*
	* @param InWorld						World to create the foliage instance in
	* @param bCreateIfNone					Create if doesnt already exist
	* returns								pointer to foliage object instance
	*/
	static FOLIAGE_API AInstancedFoliageActor* GetInstancedFoliageActorForCurrentLevel(const UWorld* InWorld, bool bCreateIfNone = false);


	/**
	* Get the instanced foliage actor for the specified streaming level.
	* @param InLevel						Level to create the foliage instance in
	* @param bCreateIfNone					Create if doesnt already exist
	* returns								pointer to foliage object instance
	*/
	static FOLIAGE_API AInstancedFoliageActor* GetInstancedFoliageActorForLevel(ULevel* Level, bool bCreateIfNone = false);
						
#if WITH_EDITOR
	/**
	 * Get the instanced foliage actor for the specified params
	 * @param InWorld						World to create the foliage instance in
	 * @param bCreateIfNone					Create if doesnt already exist
	 * @param InLevelHint					Level hint for foliage instance creation
	 * @param InLocationHint				Location hint for foliage instance creation
	 */
	static FOLIAGE_API AInstancedFoliageActor* Get(UWorld* InWorld, bool bCreateIfNone, ULevel* InLevelHint = nullptr, const FVector& InLocationHint = FVector(ForceInitToZero));

	static FOLIAGE_API AInstancedFoliageActor* GetDefault(UWorld* InWorld);

	static FOLIAGE_API void UpdateInstancePartitioning(UWorld* InWorld);
	static FOLIAGE_API void MoveSelectedInstancesToActorEditorContext(UWorld* InWorld);

	static FOLIAGE_API bool FoliageTrace(const UWorld* InWorld, FHitResult& OutHit, const FDesiredFoliageInstance& DesiredInstance, FName InTraceTag = NAME_None, bool InbReturnFaceIndex = false, const FFoliageTraceFilterFunc& FilterFunc = FFoliageTraceFilterFunc(), bool bAverageNormal = false);
	static FOLIAGE_API bool CheckCollisionWithWorld(const UWorld* InWorld, const UFoliageType* Settings, const FFoliageInstance& Inst, const FVector& HitNormal, const FVector& HitLocation, UPrimitiveComponent* HitComponent);

	virtual void PreEditUndo() override;
	virtual void PostEditUndo() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual bool ShouldExport() override;
	virtual bool ShouldImport(FStringView ActorPropString, bool IsMovingLevel) override;

	// Called in response to BSP rebuilds to migrate foliage from obsolete to new components.
	FOLIAGE_API void MapRebuild();

	// Moves instances based on the specified component to the current streaming level
	static FOLIAGE_API void MoveInstancesForComponentToCurrentLevel(UActorComponent* InComponent);
	static FOLIAGE_API void MoveInstancesForComponentToLevel(UActorComponent* InComponent, ULevel* TargetLevel);

	// Change all instances based on one component to a new component (possible in another level).
	// The instances keep the same world locations
	FOLIAGE_API void MoveInstancesToNewComponent(UPrimitiveComponent* InOldComponent, UPrimitiveComponent* InNewComponent);
	FOLIAGE_API void MoveInstancesToNewComponent(UPrimitiveComponent* InOldComponent, const FBox& InBoxWithInstancesToMove, UPrimitiveComponent* InNewComponent);
	static FOLIAGE_API void MoveInstancesToNewComponent(UWorld* InWorld, UPrimitiveComponent* InOldComponent, const FBox& InBoxWithInstancesToMove, UPrimitiveComponent* InNewComponent);
	static FOLIAGE_API void MoveInstancesToNewComponent(UWorld* InWorld, UPrimitiveComponent* InOldComponent, UPrimitiveComponent* InNewComponent);
	
	// Move selected instances to a foliage actor in target level
	FOLIAGE_API void MoveSelectedInstancesToLevel(ULevel* InTargetLevel);

	// Move all instances to a foliage actor in target level
	FOLIAGE_API void MoveAllInstancesToLevel(ULevel* InTargetLevel);
	
	// Move instances to a foliage actor in target level
	FOLIAGE_API void MoveInstancesToLevel(ULevel* InTargetLevel, TSet<int32>& InInstanceList, FFoliageInfo* InCurrentMeshInfo, UFoliageType* InFoliageType, bool bSelect = false);

	// Returns a map of Static Meshes and their placed instances attached to a component.
	FOLIAGE_API TMap<UFoliageType*, TArray<const FFoliageInstancePlacementInfo*>> GetInstancesForComponent(UActorComponent* InComponent);

	// Deletes the instances attached to a component
	FOLIAGE_API void DeleteInstancesForComponent(UActorComponent* InComponent);
	FOLIAGE_API void DeleteInstancesForComponent(UActorComponent* InComponent, const UFoliageType* InFoliageType);

	// Deletes the instances attached to a component, traverses all foliage actors in the world
	static FOLIAGE_API void DeleteInstancesForComponent(UWorld* InWorld, UActorComponent* InComponent);

	// Deletes the instances spawned by a procedural component
	FOLIAGE_API bool DeleteInstancesForProceduralFoliageComponent(const UProceduralFoliageComponent* InProceduralFoliageComponent, bool bInRebuildTree);
	FOLIAGE_API bool DeleteInstancesForProceduralFoliageComponent(const FGuid& InProceduralGuid, bool bInRebuildTree);
	FOLIAGE_API bool DeleteInstancesForAllProceduralFoliageComponents(bool bInRebuildTree);

	/** @return True if any instances exist that were spawned by the given procedural component */
	FOLIAGE_API bool ContainsInstancesFromProceduralFoliageComponent(const UProceduralFoliageComponent* InProceduralFoliageComponent);
	FOLIAGE_API bool ContainsInstancesFromProceduralFoliageComponent(const FGuid& InProceduralGuid);

	// Finds a mesh entry or adds it if it doesn't already exist
	FOLIAGE_API FFoliageInfo* FindOrAddMesh(UFoliageType* InType);

	FOLIAGE_API UFoliageType* AddFoliageType(const UFoliageType* InType, FFoliageInfo** OutInfo = nullptr);
	// Add a new static mesh.
	FOLIAGE_API FFoliageInfo* AddMesh(UStaticMesh* InMesh, UFoliageType** OutSettings = nullptr, const UFoliageType_InstancedStaticMesh* DefaultSettings = nullptr);
	FOLIAGE_API FFoliageInfo* AddMesh(UFoliageType* InType);

	// Remove the FoliageType from the list, and all its instances.
	FOLIAGE_API void RemoveFoliageType(UFoliageType** InFoliageType, int32 Num);

	// Select an individual instance.
	FOLIAGE_API void SelectInstance(UInstancedStaticMeshComponent* InComponent, int32 InComponentInstanceIndex, bool bToggle);

	// Select an individual instance.
	FOLIAGE_API bool SelectInstance(AActor* InActor, bool bToggle);

	//Get the bounds of all selected instances
	FOLIAGE_API FBox GetSelectionBoundingBox() const;

	// Whether actor has selected instances
	FOLIAGE_API bool HasSelectedInstances() const;

	// Will return all the foliage type used by currently selected instances
	FOLIAGE_API TMap<UFoliageType*, FFoliageInfo*> GetSelectedInstancesFoliageType();

	// Returns FoliageType associated to this FoliageInfo
	FOLIAGE_API const UFoliageType* FindFoliageType(const FFoliageInfo* InFoliageInfo) const;

	// Will return all the foliage type used
	FOLIAGE_API TMap<UFoliageType*, FFoliageInfo*> GetAllInstancesFoliageType();

	// Propagate the selected instances to the actual foliage implementation
	FOLIAGE_API void ApplySelection(bool bApply);

	// Returns the location for the widget
	FOLIAGE_API bool GetSelectionLocation(FBox& OutLocation) const;

	/** Whether there any foliage instances painted on specified component */
	static FOLIAGE_API bool HasFoliageAttached(UActorComponent* InComponent);

	/* Called to notify InstancedFoliageActor that a UFoliageType has been modified */
	FOLIAGE_API void NotifyFoliageTypeChanged(UFoliageType* FoliageType, bool bSourceChanged);
	void NotifyFoliageTypeWillChange(UFoliageType* FoliageType);

	DECLARE_EVENT_OneParam(AInstancedFoliageActor, FOnFoliageTypeMeshChanged, UFoliageType*);
	FOnFoliageTypeMeshChanged& OnFoliageTypeMeshChanged() { return OnFoliageTypeMeshChangedEvent; }

	/* Fix up a duplicate IFA */
	void RepairDuplicateIFA(AInstancedFoliageActor* InDuplicateIFA);

	void RemoveBaseComponentOnFoliageTypeInstances(UFoliageType* FoliageType);

	UFUNCTION(BlueprintCallable, Category="Foliage", meta = (WorldContext = "WorldContextObject"))
	static void AddInstances(UObject* WorldContextObject, UFoliageType* InFoliageType, const TArray<FTransform>& InTransforms);

	UFUNCTION(BlueprintCallable, Category="Foliage", meta = (WorldContext = "WorldContextObject"))
	static void RemoveAllInstances(UObject* WorldContextObject, UFoliageType* InFoliageType);
#endif	//WITH_EDITOR

private:
#if WITH_EDITORONLY_DATA
	// Deprecated data, will be converted and cleaned up in PostLoad
	TMap<UFoliageType*, TUniqueObj<struct FFoliageMeshInfo_Deprecated>> FoliageMeshes_Deprecated;
	TMap<UFoliageType*, TUniqueObj<struct FFoliageMeshInfo_Deprecated2>> FoliageMeshes_Deprecated2;
#endif//WITH_EDITORONLY_DATA
	
#if WITH_EDITOR
	void ClearSelection();
	void MoveInstancesToNewComponent(UPrimitiveComponent* InOldComponent, UPrimitiveComponent* InNewComponent, TFunctionRef<TArray<int32>(const FFoliageInfo&)> GetInstancesToMoveFunc);
	bool DeleteInstancesForProceduralFoliageComponentInternal(const FGuid& InProceduralGuid, bool bInRebuildTree, bool bInDeleteAll);
	
	friend class UFoliageEditorSubsystem;
	FOLIAGE_API bool MoveInstancesForMovedComponent(UActorComponent* InComponent);
	FOLIAGE_API void UpdateInstancePartitioningForMovedComponent(UActorComponent* InComponent);
	
	FOLIAGE_API void MoveInstancesForMovedOwnedActors(AActor* InActor);

	FOLIAGE_API void UpdateFoliageActorInstance(AActor* InActor);
	FOLIAGE_API void DeleteFoliageActorInstance(AActor* InActor);

	FOLIAGE_API void PostApplyLevelOffset(const FVector& InOffset, bool bWorldShift);
	FOLIAGE_API void PostApplyLevelTransform(const FTransform& InTransform);
#endif
private:
#if WITH_EDITOR
	FOnFoliageTypeMeshChanged OnFoliageTypeMeshChangedEvent;
#endif

};
