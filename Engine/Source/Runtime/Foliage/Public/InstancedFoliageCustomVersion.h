// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

// Custom serialization version for all packages containing Instance Foliage
struct FFoliageCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,
		// Converted to use HierarchicalInstancedStaticMeshComponent
		FoliageUsingHierarchicalISMC = 1,
		// Changed Component to not RF_Transactional
		HierarchicalISMCNonTransactional = 2,
		// Added FoliageTypeUpdateGuid
		AddedFoliageTypeUpdateGuid = 3,
		// Use a GUID to determine whic procedural actor spawned us
		ProceduralGuid = 4,
		// Support for cross-level bases 
		CrossLevelBase = 5,
		// FoliageType for details customization
		FoliageTypeCustomization = 6,
		// FoliageType for details customization continued
		FoliageTypeCustomizationScaling = 7,
		// FoliageType procedural scale and shade settings updated
		FoliageTypeProceduralScaleAndShade = 8,
		// Added FoliageHISMC and blueprint support
		FoliageHISMCBlueprints = 9,
		// Added Mobility setting to UFoliageType
		AddedMobility = 10,
		// Make sure that foliage has FoliageHISMC class
		FoliageUsingFoliageISMC = 11,
		// Foliage Actor Support
		FoliageActorSupport = 12,
		// Foliage Actor (No weak ptr)
		FoliageActorSupportNoWeakPtr = 13,
		// Foliage Instances are now always saved local to Level
		FoliageRepairInstancesWithLevelTransform = 14,
		// Supports discarding foliage types on load independently from density scaling
		FoliageDiscardOnLoad = 15,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FFoliageCustomVersion() {}
};