// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/StaticMeshActor.h"
#include "InteractiveFoliageActor.generated.h"

class UPrimitiveComponent;

UCLASS(MinimalAPI)
class AInteractiveFoliageActor : public AStaticMeshActor
{
	GENERATED_UCLASS_BODY()

private:
	/** Collision cylinder */
	UPROPERTY()
	TObjectPtr<class UCapsuleComponent> CapsuleComponent;

	/**
	 * Position of the last actor to enter the collision cylinder.
	 * This currently does not handle multiple actors affecting the foliage simultaneously.
	 */
	UPROPERTY(transient)
	FVector TouchingActorEntryPosition;

	/** Simulated physics state */
	UPROPERTY(transient)
	FVector FoliageVelocity;

	/** @todo document */
	UPROPERTY(transient)
	FVector FoliageForce;

	/** @todo document */
	UPROPERTY(transient)
	FVector FoliagePosition;

public:
	/** Scales forces applied from damage events. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FoliagePhysics)
	float FoliageDamageImpulseScale;

	/** Scales forces applied from touch events. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FoliagePhysics)
	float FoliageTouchImpulseScale;

	/** Determines how strong the force that pushes toward the spring's center will be. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FoliagePhysics)
	float FoliageStiffness;

	/**
	 * Same as FoliageStiffness, but the strength of this force increases with the square of the distance to the spring's center.
	 * This force is used to prevent the spring from extending past a certain point due to touch and damage forces.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FoliagePhysics)
	float FoliageStiffnessQuadratic;

	/**
	 * Determines the amount of energy lost by the spring as it oscillates.
	 * This force is similar to air friction.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FoliagePhysics)
	float FoliageDamping;

	/** Clamps the magnitude of each damage force applied. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FoliagePhysics)
	float MaxDamageImpulse;

	/** Clamps the magnitude of each touch force applied. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FoliagePhysics)
	float MaxTouchImpulse;

	/** Clamps the magnitude of combined forces applied each update. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FoliagePhysics)
	float MaxForce;

	//@todo - hook this up	/** @todo document */
	UPROPERTY()
	float Mass;

protected:
	/** @todo document */
	void SetupCollisionCylinder();

	/** Called when capsule is touched */
	UFUNCTION()
	void CapsuleTouched(UPrimitiveComponent* OverlappedComp, AActor* Other, UPrimitiveComponent* OtherComp, int32 OtherBodyIndex, bool bFromSweep, const FHitResult& OverlapInfo);
public:
	//~ Begin AActor Interface
	virtual void Tick(float DeltaSeconds) override;
	virtual void PostActorCreated() override;
	virtual float TakeDamage(float DamageAmount, struct FDamageEvent const& DamageEvent, class AController* EventInstigator, class AActor* DamageCauser) override;
	//~ End AActor Interface

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	//~ End UObject Interface

};



