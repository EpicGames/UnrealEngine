// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "FoliageType.h"
#include "ProceduralFoliageBlockingVolume.generated.h"

class AProceduralFoliageVolume;

/** An invisible volume used to block ProceduralFoliage instances from being spawned. */
UCLASS(MinimalAPI)
class AProceduralFoliageBlockingVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(Category = ProceduralFoliage, EditAnywhere)
	TObjectPtr<AProceduralFoliageVolume> ProceduralFoliageVolume;

	UPROPERTY(Category = ProceduralFoliage, EditAnywhere)
	FFoliageDensityFalloff DensityFalloff;
};



