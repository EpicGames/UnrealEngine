// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ProceduralFoliageInstance.h"
#include "GenericQuadTree.h"

class FProceduralFoliageBroadphase
{
public:
	FProceduralFoliageBroadphase(float TileSize = 0.f, float MinimumQuadTreeSize = 100.f);
	FProceduralFoliageBroadphase(const FProceduralFoliageBroadphase& OtherBroadphase);
	FProceduralFoliageBroadphase& operator=(const FProceduralFoliageBroadphase& OtherBroadphase);

	bool TestAgainstAABB(FProceduralFoliageInstance* Instance);
	void Insert(FProceduralFoliageInstance* Instance);
	bool GetOverlaps(FProceduralFoliageInstance* Instance, TArray<FProceduralFoliageOverlap>& Overlaps) const;
	void GetInstancesInBox(const FBox2D& Box, TArray<FProceduralFoliageInstance*>& Instances) const;
	void Remove(FProceduralFoliageInstance* Instance);
	void Empty();

private:
	TQuadTree<FProceduralFoliageInstance*, 4> QuadTree;
};
