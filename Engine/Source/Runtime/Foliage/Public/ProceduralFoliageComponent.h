// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Components/ActorComponent.h"
#include "InstancedFoliage.h"
#include "ProceduralFoliageComponent.generated.h"

class AVolume;
class UProceduralFoliageSpawner;
class UDataLayerAsset;
struct FBodyInstance;

/** Describes the layout of the tiles used for procedural foliage simulation */
struct FTileLayout
{
	FTileLayout()
		: BottomLeftX(0), BottomLeftY(0), NumTilesX(0), NumTilesY(0), HalfHeight(0.f)
	{
	}

	// The X coordinate (in whole tiles) of the bottom-left-most active tile
	int32 BottomLeftX;

	// The Y coordinate (in whole tiles) of the bottom-left-most active tile
	int32 BottomLeftY;

	// The total number of active tiles along the x-axis
	int32 NumTilesX;

	// The total number of active tiles along the y-axis
	int32 NumTilesY;
	

	FVector::FReal HalfHeight;
};

UCLASS(BlueprintType, MinimalAPI)
class UProceduralFoliageComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

	/** The procedural foliage spawner used to generate foliage instances within this volume. */
	UPROPERTY(Category = "ProceduralFoliage", BlueprintReadWrite, EditAnywhere)
	TObjectPtr<UProceduralFoliageSpawner> FoliageSpawner;

	/** The amount of overlap between simulation tiles (in cm). */
	UPROPERTY(Category = "ProceduralFoliage", BlueprintReadWrite, EditAnywhere)
	float TileOverlap;

#if WITH_EDITORONLY_DATA

	/** Whether to place foliage on landscape */
	UPROPERTY(Category = "ProceduralFoliage", BlueprintReadWrite, EditAnywhere)
	bool bAllowLandscape;

	/** Whether to place foliage on BSP */
	UPROPERTY(Category = "ProceduralFoliage", BlueprintReadWrite, EditAnywhere)
	bool bAllowBSP;

	/** Whether to place foliage on StaticMesh */
	UPROPERTY(Category = "ProceduralFoliage", BlueprintReadWrite, EditAnywhere)
	bool bAllowStaticMesh;

	/** Whether to place foliage on translucent geometry */
	UPROPERTY(Category = "ProceduralFoliage", BlueprintReadWrite, EditAnywhere)
	bool bAllowTranslucent;

	/** Whether to place foliage on other blocking foliage geometry */
	UPROPERTY(Category = "ProceduralFoliage", BlueprintReadWrite, EditAnywhere)
	bool bAllowFoliage;

	/** Whether to visualize the tiles used for the foliage spawner simulation */
	UPROPERTY(Category = "ProceduralFoliage", BlueprintReadWrite, EditAnywhere)
	bool bShowDebugTiles;

#endif

	struct FGenerateProceduralContentParams
	{
		FGenerateProceduralContentParams()
			: FoliageSpawner(nullptr)
			, Bounds(EForceInit::ForceInit)
			, TileOverlap(0.f)
			, ProceduralVolumeInstance(nullptr)
		{
		}

		TObjectPtr<UProceduralFoliageSpawner> FoliageSpawner;
		FBox Bounds;
		float TileOverlap;
		FGuid ProceduralGuid;
		FBodyInstance* ProceduralVolumeInstance;
	};
				
	// UObject interface
	FOLIAGE_API virtual void PostEditImport() override;

	/**
	 * Returns a params struct based on this component properties. 
	 */
	FOLIAGE_API FGenerateProceduralContentParams GetGenerateProceduralContentParams() const;

	/**
	 * Runs the procedural foliage simulation, removes the old result, creates instances with the new result
	 * @return True if the simulation succeeded
	 */ 
	FOLIAGE_API bool ResimulateProceduralFoliage(TFunctionRef<void(const TArray<FDesiredFoliageInstance>&)> AddInstancesFunc);

	/** 
	 * Runs the procedural foliage simulation to generate a list of desired instances to spawn.
	 * @return True if the simulation succeeded
	 */
	FOLIAGE_API bool GenerateProceduralContent(TArray<FDesiredFoliageInstance>& OutInstances);
	static FOLIAGE_API bool GenerateProceduralContent(const FGenerateProceduralContentParams& InParams, TArray<FDesiredFoliageInstance>& OutInstances);
			
	/** Removes all spawned foliage instances in the level that were spawned by this component */
	FOLIAGE_API void RemoveProceduralContent(bool bInRebuildTree = true);
	static FOLIAGE_API void RemoveProceduralContent(UWorld* InWorld, const FGuid& InProceduralGuid, bool bInRebuildTree, TSet<AInstancedFoliageActor*>& OutModifiedActors);

	/** @return True if any foliage instances in the level were spawned by this component */
	FOLIAGE_API bool HasSpawnedAnyInstances();
	
	/** @return The position in world space of the bottom-left corner of the bottom-left-most active tile */
	FOLIAGE_API FVector GetWorldPosition() const;
	static FOLIAGE_API FVector GetWorldPosition(const FGenerateProceduralContentParams& Param);

	/** @return The bounds of area encompassed by the simulation */
	FOLIAGE_API virtual FBox GetBounds() const;

	/** @return The body instanced used for bounds checking */
	FOLIAGE_API FBodyInstance* GetBoundsBodyInstance() const;

	/** Determines the basic layout of the tiles used in the simulation */
	FOLIAGE_API void GetTileLayout(FTileLayout& OutTileLayout) const;
	static FOLIAGE_API void GetTileLayout(const FGenerateProceduralContentParams& Params, FTileLayout& OutTileLayout);

	void SetSpawningVolume(AVolume* InSpawningVolume) { SpawningVolume = InSpawningVolume; }
	const FGuid& GetProceduralGuid() const { return ProceduralGuid; }

#if WITH_EDITOR
	FOLIAGE_API void LoadSimulatedRegion();
	FOLIAGE_API void UnloadSimulatedRegion();
	FOLIAGE_API bool IsSimulatedRegionLoaded();
#endif

private:
	UPROPERTY()
	TObjectPtr<AVolume> SpawningVolume;
	
	UPROPERTY()
	FGuid ProceduralGuid;

#if WITH_EDITORONLY_DATA
	friend class UProceduralFoliageEditorLibrary;
#endif
};
