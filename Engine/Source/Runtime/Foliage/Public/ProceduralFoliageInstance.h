// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EngineDefines.h"
#include "ProceduralFoliageInstance.generated.h"

class UActorComponent;
class UFoliageType;
struct FProceduralFoliageInstance;

UENUM(BlueprintType)
namespace ESimulationOverlap
{
	enum Type : int
	{
		/*Instances overlap with collision*/
		CollisionOverlap,
		/*Instances overlap with shade*/
		ShadeOverlap,
		/*No overlap*/
		None
	};
}

UENUM(BlueprintType)
namespace ESimulationQuery
{
	enum Type : int
	{
		None = 0 UMETA(Hidden),
		/*Instances overlap with collision*/
		CollisionOverlap = 1,
		/*Instances overlap with shade*/
		ShadeOverlap = 2,
		/*any overlap*/
		AnyOverlap = 3
	};
}

struct FProceduralFoliageInstance;
struct FProceduralFoliageOverlap
{
	FProceduralFoliageOverlap(FProceduralFoliageInstance* InA, FProceduralFoliageInstance* InB, ESimulationOverlap::Type InType)
	: A(InA)
	, B(InB)
	, OverlapType(InType)
	{}

	FProceduralFoliageInstance* A;
	FProceduralFoliageInstance* B;
	ESimulationOverlap::Type OverlapType;
};

USTRUCT(BlueprintType)
struct FProceduralFoliageInstance
{
public:
	GENERATED_USTRUCT_BODY()
	FOLIAGE_API FProceduralFoliageInstance();

	static FOLIAGE_API FProceduralFoliageInstance* Domination(FProceduralFoliageInstance* A, FProceduralFoliageInstance* B, ESimulationOverlap::Type OverlapType);

	FOLIAGE_API float GetMaxRadius() const;
	FOLIAGE_API float GetShadeRadius() const;
	FOLIAGE_API float GetCollisionRadius() const;

	bool IsAlive() const { return bAlive; }

	FOLIAGE_API void TerminateInstance();

public:
	UPROPERTY()
	FQuat Rotation;

	UPROPERTY(Category = ProceduralFoliageInstance, EditAnywhere, BlueprintReadWrite)
	FVector Location;

	UPROPERTY(Category = ProceduralFoliageInstance, EditAnywhere, BlueprintReadWrite)
	float Age;

	UPROPERTY(Category = ProceduralFoliageInstance, EditAnywhere, BlueprintReadWrite)
	FVector Normal;

	UPROPERTY()
	float Scale;

	UPROPERTY()
	TObjectPtr<const UFoliageType> Type;

	UActorComponent* BaseComponent;

	bool bBlocker;
private:
	bool bAlive;
};
