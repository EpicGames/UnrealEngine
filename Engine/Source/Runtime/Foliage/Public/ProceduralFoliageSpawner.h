// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Math/RandomStream.h"
#include "ProceduralFoliageInstance.h"
#include "FoliageTypeObject.h"
#include "ProceduralFoliageSpawner.generated.h"

class UProceduralFoliageTile;

UCLASS(BlueprintType, Blueprintable, MinimalAPI)
class UProceduralFoliageSpawner : public UObject
{
	GENERATED_UCLASS_BODY()

	/** The seed used for generating the randomness of the simulation. */
	UPROPERTY(Category = ProceduralFoliageSimulation, EditAnywhere, BlueprintReadOnly)
	int32 RandomSeed;

	/** Length of the tile (in cm) along one axis. The total area of the tile will be TileSize*TileSize. */
	UPROPERTY(Category = ProceduralFoliageSimulation, EditAnywhere, BlueprintReadOnly)
	float TileSize;

	/** The number of unique tiles to generate. The final simulation is a procedurally determined combination of the various unique tiles. */
	UPROPERTY(Category = ProceduralFoliageSimulation, EditAnywhere, BlueprintReadOnly)
	int32 NumUniqueTiles;

	/** Minimum size of the quad tree used during the simulation. Reduce if too many instances are in splittable leaf quads (as warned in the log). */
	UPROPERTY(Category = ProceduralFoliageSimulation, EditAnywhere, BlueprintReadOnly)
	float MinimumQuadTreeSize;

	FThreadSafeCounter LastCancel;

private:
	/** The types of foliage to procedurally spawn. */
	UPROPERTY(Category = ProceduralFoliageSimulation, EditAnywhere)
	TArray<FFoliageTypeObject> FoliageTypes;

	/**
	 * If checked, will override the default behavior of using the global foliage material list and instead use the Override Foliage Terrain Materials defined here.
	 * If the override is used, you must manually provide ALL materials this procedural foliage spawner should spawn on top of.
	 */
	UPROPERTY(Category = ProceduralFoliageSimulation, EditAnywhere)
	bool bUseOverrideFoliageTerrainMaterials = false;

	/** Procedural foliage will only spawn on materials specified here. These are only used if 'Use Override Foliage Terrain Materials' is checked. */
	UPROPERTY(Category = ProceduralFoliageSimulation, EditAnywhere)
	TArray<TSoftObjectPtr<UMaterialInterface>> OverrideFoliageTerrainMaterials;

public:
	UFUNCTION(BlueprintCallable, Category = ProceduralFoliageSimulation)
	FOLIAGE_API void Simulate(int32 NumSteps = -1);
	FOLIAGE_API void Empty();

	FOLIAGE_API int32 GetRandomNumber();

	const TArray<FFoliageTypeObject>& GetFoliageTypes() const { return FoliageTypes; }

	bool UsesOverrideFoliageTerrainMaterials() const { return bUseOverrideFoliageTerrainMaterials; }

	const TArray<TSoftObjectPtr<UMaterialInterface>>& GetFoliageTerrainMaterials() const { return OverrideFoliageTerrainMaterials; }

	/** Returns the instances that need to spawn for a given min,max region */
	FOLIAGE_API void GetInstancesToSpawn(TArray<FProceduralFoliageInstance>& OutInstances, const FVector& Min = FVector(-FLT_MAX, -FLT_MAX, -FLT_MAX), const FVector& Max = FVector(FLT_MAX, FLT_MAX, FLT_MAX) ) const;

	FOLIAGE_API virtual void Serialize(FArchive& Ar);

	/** Takes a tile index and returns a random tile associated with that index. */
	FOLIAGE_API const UProceduralFoliageTile* GetRandomTile(int32 X, int32 Y);

	/** Creates a temporary empty tile with the appropriate settings created for it. */
	FOLIAGE_API UProceduralFoliageTile* CreateTempTile();

private:
	FOLIAGE_API void CreateProceduralFoliageInstances();

	FOLIAGE_API void SetClean();
private:
	TArray<TWeakObjectPtr<UProceduralFoliageTile>> PrecomputedTiles;

	FRandomStream RandomStream;
};
