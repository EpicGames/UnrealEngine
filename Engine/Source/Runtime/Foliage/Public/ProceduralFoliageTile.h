// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Math/RandomStream.h"
#include "InstancedFoliage.h"
#include "ProceduralFoliageInstance.h"
#include "ProceduralFoliageBroadphase.h"
#include "ProceduralFoliageTile.generated.h"

class UFoliageType;
class UProceduralFoliageSpawner;
struct FBodyInstance;

/**
 *	Procedurally determines where to spawn foliage meshes within a discrete area.
 *	Generally, a procedural foliage simulation as a whole is composed of multiple tiles.
 *	Tiles are able to overlap one another as well to create a seamless appearance.
 *	
 *	Note that the tile is not responsible for actually spawning any instances, it only determines where they should be placed.
 *	Following a simulation, call ExtractDesiredInstances for information about where each instance should spawn.
 *	
 *	Note also that, barring any core changes to the ordering of TSet, foliage generation is deterministic 
 *	(i.e. given the same inputs, the result of the simulation will always be the same).
 */
UCLASS(MinimalAPI)
class UProceduralFoliageTile : public UObject
{
	GENERATED_UCLASS_BODY()

	/**
	 *	Procedurally determine the placement of foliage instances.
	 *	@param InFoliageSpawner the foliage spawner containing the various foliage types to simulate
	 *	@param RandomSeed For seeding (in the RNG sense) the random dispersion of seeds (in the foliage sense)
	 *	@param MaxNumSteps The maximum number of steps to run the simulation
	 */
	FOLIAGE_API void Simulate(const UProceduralFoliageSpawner* InFoliageSpawner, const int32 RandomSeed, const int32 MaxNumSteps, const int32 InLastCancel);
	
	/**
	 *	Extracts information on each instance that should be spawned based on the simulation.
	 */
	FOLIAGE_API void ExtractDesiredInstances(TArray<FDesiredFoliageInstance>& OutDesiredInstances, const FTransform& WorldTM, const FVector2D& ActorVolumeLocation, FVector::FReal ActorVolumeMaxExtent, const FGuid& ProceduralGuid, const FVector::FReal HalfHeight, const FBodyInstance* VolumeBodyInstance, bool bEmptyTileInfo = true);
	
	/**
	 *	Copies instances within this tile to another
	 *	@param ToTile The tile to copy instances to
	 *	@param LocalAABB The region (in local space) to copy instances from
	 *	@param RelativeTM The transform of the destination tile relative to this tile
	 *	@param Overlap The amount by which the tiles are overlapping
	 */
	FOLIAGE_API void CopyInstancesToTile(UProceduralFoliageTile* ToTile, const FBox2D& LocalAABB, const FTransform& RelativeTM, const float Overlap) const;

	/**
	 *	Removes a single instance from the tile
	 *	@param Inst The instance to remove
	 */
	FOLIAGE_API void RemoveInstance(FProceduralFoliageInstance* Inst);

	/** Removes all instances from the tile */
	FOLIAGE_API void RemoveInstances();

	/** Empty arrays and sets */
	FOLIAGE_API void Empty();

	/** Initializes the tile for simulation. Called automatically by Simulate, so only bother to call if the tile won't actually simulate. */
	FOLIAGE_API void InitSimulation(const UProceduralFoliageSpawner* InFoliageSpawner, const int32 InRandomSeed);

	// UObject interface
	FOLIAGE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	FOLIAGE_API virtual void BeginDestroy() override;

private:
	
	/**
	* Finds all the instances within an AABB. The instances are returned in the space local to the AABB.
	* That is, an instance in the bottom left AABB will have a coordinate of (0,0).
	*
	* @param LocalAABB The AABB to get instances within
	* @param OutInstances Contains the instances within the specified AABB
	* @param bFullyContainedOnly If true, will only get instances that are fully contained within the AABB
	*/
	void GetInstancesInAABB(const FBox2D& LocalAABB, TArray<FProceduralFoliageInstance* >& OutInstances, bool bFullyContainedOnly = true) const;

	/**
	 * Adds procedural instances to the tile from some external source.
	 * @param The new instances to add to the tile
	 * @param ToLocalTM Used to transform the new instances to this tile's local space
	 * @param InnerLocalAABB New instances outside this region will be tracked for blocking purposes only (and never spawned)
	 */
	void AddInstances(const TArray<FProceduralFoliageInstance*>& NewInstances, const FTransform& ToLocalTM, const FBox2D& InnerLocalAABB);

	/** Fills the InstancesArray by iterating through the contents of the InstancesSet */
	void InstancesToArray();

	/** Run the simulation for the appropriate number of steps. 
	 * @param bOnlyInShade Whether the simulation should run exclusively within existing shade from other instances
	 */
	void RunSimulation(const int32 MaxNumSteps, bool bOnlyInShade);

	/** Advance the simulation by a step (think of a step as a generation) */
	void StepSimulation();

	/** Randomly seed the next step of the simulation */
	void AddRandomSeeds(TArray<FProceduralFoliageInstance*>& OutInstances);

	/** Determines whether the instance will survive all the overlaps, and then kills the appropriate instances. Returns true if the instance survives */
	bool HandleOverlaps(FProceduralFoliageInstance* Instance);

	/** Attempts to create a new instance and resolves any overlaps. Returns the new instance if successful for calling code to add to Instances */
	FProceduralFoliageInstance* NewSeed(const FVector& Location, float Scale, const UFoliageType* Type, float InAge, bool bBlocker = false);

	void SpreadSeeds(TArray<FProceduralFoliageInstance*>& NewSeeds);
	void AgeSeeds();

	void MarkPendingRemoval(FProceduralFoliageInstance* ToRemove);
	void FlushPendingRemovals();

	bool UserCancelled() const;
	
private:

	UPROPERTY()
	TObjectPtr<const UProceduralFoliageSpawner> FoliageSpawner;

	TSet<FProceduralFoliageInstance*> PendingRemovals;
	TSet<FProceduralFoliageInstance*> InstancesSet;

	UPROPERTY()
	TArray<FProceduralFoliageInstance> InstancesArray;

	int32 SimulationStep;
	FProceduralFoliageBroadphase Broadphase;	

	int32 RandomSeed;
	FRandomStream RandomStream;
	bool bSimulateOnlyInShade;

	int32 LastCancel;

private:
	float GetRandomGaussian();
	FVector GetSeedOffset(const UFoliageType* Type, float MinDistance);
};
