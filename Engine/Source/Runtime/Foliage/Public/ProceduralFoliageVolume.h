// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"
#include "ProceduralFoliageVolume.generated.h"

class UProceduralFoliageComponent;
class FLoaderAdapterActor;

UCLASS(MinimalAPI)
class AProceduralFoliageVolume: public AVolume, public IWorldPartitionActorLoaderInterface
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(Category = ProceduralFoliage, VisibleAnywhere, BlueprintReadOnly)
	TObjectPtr<UProceduralFoliageComponent> ProceduralComponent;

#if WITH_EDITOR
	//~ Begin AActor Interface
	FOLIAGE_API virtual void PostRegisterAllComponents();
	//~ End AActor Interface

	//~ Begin UObject Interface
	FOLIAGE_API virtual void BeginDestroy() override;
	FOLIAGE_API virtual void PostEditImport() override;
	//~ End UObject Interface

	FOLIAGE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;

	//~ Begin IWorldPartitionActorLoaderInterface interface
	FOLIAGE_API virtual ILoaderAdapter* GetLoaderAdapter() override;
	//~ End IWorldPartitionActorLoaderInterface interface

private:
	FLoaderAdapterActor* WorldPartitionActorLoader;
#endif
};
